{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "GameplayTagsEditor",
	"Description" : "GameplayTagsEditor provides blueprint nodes and editor UI to enable the use of GameplayTags for tagging assets and objects",
	"Category" : "Editor",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "GameplayTagsEditor",
			"Type" : "UncookedOnly",
			"LoadingPhase" : "PreDefault"
		}
	]
}