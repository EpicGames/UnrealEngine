// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node.h"
#include "GameplayTagsK2Node_LiteralGameplayTag.generated.h"

enum class EEdGraphNodeDeprecationType;
namespace ENodeTitleType { enum Type : int; }
struct FEdGraphNodeDeprecationResponse;

class FBlueprintActionDatabaseRegistrar;
class UEdGraph;
class UEdGraphSchema;

UCLASS()
class UGameplayTagsK2Node_LiteralGameplayTag : public UK2Node
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR
	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual bool CanDuplicateNode() const override { return false; }
	virtual bool NodeCausesStructuralBlueprintChange() const override { return true; }
	virtual bool CanCreateUnderSpecifiedSchema(const UEdGraphSchema* Schema) const override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
	virtual bool ShouldShowNodeProperties() const override { return true; }
	virtual bool IsNodeSafeToIgnore() const override { return true; }
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	virtual bool IsNodePure() const override { return true; }
	virtual bool IsDeprecated() const override { return true; } 
	virtual void ConvertDeprecatedNode(UEdGraph* Graph, bool bOnlySafeChanges) override;
	virtual FEdGraphNodeDeprecationResponse GetDeprecationResponse(EEdGraphNodeDeprecationType DeprecationType) const override;
	//~ End UK2Node Interface
#endif

};
