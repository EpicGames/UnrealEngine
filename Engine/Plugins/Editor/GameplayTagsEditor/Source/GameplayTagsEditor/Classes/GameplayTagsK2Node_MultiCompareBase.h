// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "K2Node.h"
#include "GameplayTagsK2Node_MultiCompareBase.generated.h"

UCLASS()
class UGameplayTagsK2Node_MultiCompareBase : public UK2Node
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = PinOptions)
	int32 NumberOfPins;

	UPROPERTY()
	TArray<FName> PinNames;

	// UObject interface
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UObject interface

	// UEdGraphNode interface
	virtual FText GetTooltipText() const override;
	virtual bool CanDuplicateNode() const override { return false; }
	// End of UEdGraphNode interface

	// UK2Node interface
	virtual bool NodeCausesStructuralBlueprintChange() const override { return true; }
	virtual bool ShouldShowNodeProperties() const override { return true; }
	virtual FText GetMenuCategory() const override;
	virtual bool IsNodePure() const override { return true; }
	// End of UK2Node interface

	void AddPin();
	void RemovePin();

protected:

	virtual void AddPinToSwitchNode(){};
	FString GetUniquePinName();
};
