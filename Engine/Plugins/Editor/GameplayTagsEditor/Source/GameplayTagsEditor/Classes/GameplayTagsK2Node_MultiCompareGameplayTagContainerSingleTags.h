// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayTagsK2Node_MultiCompareBase.h"
#include "GameplayTagsK2Node_MultiCompareGameplayTagContainerSingleTags.generated.h"

namespace ENodeTitleType { enum Type : int; }

class FBlueprintActionDatabaseRegistrar;
class UEdGraph;

UCLASS()
class UGameplayTagsK2Node_MultiCompareGameplayTagContainerSingleTags : public UGameplayTagsK2Node_MultiCompareBase
{
	GENERATED_UCLASS_BODY()

	// UEdGraphNode interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	// End of UEdGraphNode interface

	// UK2Node interface
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;		
	// End of UK2Node interface

private:

	virtual void AddPinToSwitchNode() override;
};
