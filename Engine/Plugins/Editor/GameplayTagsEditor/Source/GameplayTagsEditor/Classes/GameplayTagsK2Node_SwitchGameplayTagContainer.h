// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "K2Node_Switch.h"
#include "GameplayTagsK2Node_SwitchGameplayTagContainer.generated.h"

namespace ENodeTitleType { enum Type : int; }
struct FGameplayTagContainer;

class FBlueprintActionDatabaseRegistrar;

UCLASS(MinimalAPI)
class UGameplayTagsK2Node_SwitchGameplayTagContainer : public UK2Node_Switch
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = PinOptions)
	TArray<FGameplayTagContainer> PinContainers;

	//UPROPERTY(EditAnywhere, Category = PinOptions)
	//bool UseInputsOnGraph;

	UPROPERTY()
	TArray<FName> PinNames;

	// UObject interface
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostLoad() override;
	// End of UObject interface

	// UEdGraphNode interface
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual bool ShouldShowNodeProperties() const override { return true; }
	// End of UEdGraphNode interface

	// UK2Node interface
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	// End of UK2Node interface

	// UK2Node_Switch Interface
	virtual void AddPinToSwitchNode() override;
	virtual FEdGraphPinType GetInnerCaseType() const override;
	virtual FName GetUniquePinName() override;
	virtual FEdGraphPinType GetPinType() const override;
	// End of UK2Node_Switch Interface

	virtual FName GetPinNameGivenIndex(int32 Index) const override;

protected:
	virtual void CreateFunctionPin() override;
	virtual void CreateSelectionPin() override;
	virtual void CreateCasePins() override;
	virtual void RemovePin(UEdGraphPin* TargetPin) override;
};
