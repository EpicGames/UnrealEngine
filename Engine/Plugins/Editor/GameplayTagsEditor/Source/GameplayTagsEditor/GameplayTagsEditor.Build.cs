// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class GameplayTagsEditor : ModuleRules
	{
		public GameplayTagsEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"WorkspaceMenuStructure",
				}
			);

			PublicIncludePathModuleNames.AddRange(
				new string[] {
					"AssetTools",
					"AssetRegistry",
				}
			);


			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					// ... add private dependencies that you statically link with here ...
					"ApplicationCore",
					"Core",
					"CoreUObject",
					"Engine",
					"ApplicationCore",
					"AssetDefinition",
					"AssetTools",
					"AssetRegistry",
					"GameplayTags",
					"InputCore",
					"Slate",
					"SlateCore",
					"EditorStyle",
					"BlueprintGraph",
					"KismetCompiler",
					"GraphEditor",
					"ContentBrowser",
					"ContentBrowserData",
					"MainFrame",
					"EditorFramework",
					"UnrealEd",
					"SourceControl",
					"ToolMenus",
					"SettingsEditor",
					"ToolWidgets",
					"EditorWidgets",
				}
			);

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"PropertyEditor",
					"Settings"
				}
			);
		}
	}
}
