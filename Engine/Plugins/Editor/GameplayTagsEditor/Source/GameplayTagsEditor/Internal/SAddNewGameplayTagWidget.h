// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class SEditableTextBox;
namespace ETextCommit { enum Type : int; }
class SCheckBox;
class SSearchableComboBox;
class SNotificationItem;

/** Widget allowing the user to create new gameplay tags */
class SAddNewGameplayTagWidget : public SCompoundWidget
{
public:

	enum class EResetType : uint8
	{
		ResetAll,
		DoNotResetSource
	};

	DECLARE_DELEGATE_ThreeParams(FOnGameplayTagAdded, const FString& /*TagName*/, const FString& /*TagComment*/, const FName& /*TagSource*/);

	DECLARE_DELEGATE_RetVal_TwoParams(bool, FIsValidTag, const FString& /*TagName*/, FText* /*OutError*/)

	SLATE_BEGIN_ARGS(SAddNewGameplayTagWidget)
		: _NewTagName(TEXT(""))
		, _Padding(FMargin(15))
		, _AddButtonPadding(FMargin(0, 16, 0, 0))
		, _RestrictedTags(false)
	{}
		SLATE_EVENT(FOnGameplayTagAdded, OnGameplayTagAdded)	// Callback for when a new tag is added	
		SLATE_EVENT(FIsValidTag, IsValidTag)
		SLATE_ARGUMENT(FString, NewTagName) // String that will initially populate the New Tag Name field
		SLATE_ARGUMENT(FMargin, Padding) // Padding around the widget
		SLATE_ARGUMENT(FMargin, AddButtonPadding) // padding around the Add button
		SLATE_ARGUMENT(bool, RestrictedTags)// If we are dealing with restricted tags or regular gameplay tags
	SLATE_END_ARGS();

	GAMEPLAYTAGSEDITOR_API virtual ~SAddNewGameplayTagWidget() override;

	GAMEPLAYTAGSEDITOR_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	GAMEPLAYTAGSEDITOR_API void Construct(const FArguments& InArgs);

	/** Returns true if we're currently attempting to add a new gameplay tag to an INI file */
	bool IsAddingNewTag() const
	{
		return bAddingNewTag;
	}

	/** Begins the process of adding a subtag to a parent tag */
	void AddSubtagFromParent(const FString& ParentTagName, const FName& ParentTagSource);

	/** Begins the process of adding a duplicate of existing tag */
	void AddDuplicate(const FString& ParentTagName, const FName& ParentTagSource);

	/** Resets all input fields */
	void Reset(EResetType ResetType);

private:

	/** Sets the name of the tag. Uses the default if the name is not specified */
	void SetTagName(const FText& InName = FText());

	/** Selects tag file location. Uses the default if the location is not specified */
	void SelectTagSource(const FName& InSource = FName());

	/** Gets selected tag source as name */
	FName GetSelectedTagSource() const;

	/** Creates a list of all INIs that gameplay tags can be added to */
	void PopulateTagSources();

	/** Callback for when Enter is pressed when modifying a tag's name or comment */
	void OnCommitNewTagName(const FText& InText, ETextCommit::Type InCommitType);

	/** Callback for when the Add New Tag button is pressed */
	FReply OnAddNewTagButtonPressed();

	/** Creates a new gameplay tag and adds it to the INI files based on the widget's stored parameters */
	void CreateNewGameplayTag();

	/** Handle validation and creation of restricted tags */
	void ValidateNewRestrictedTag();
	void CreateNewRestrictedGameplayTag();
	void CancelNewTag();

	/** Populates the widget's combo box with all potential places where a gameplay tag can be stored */
	TSharedRef<SWidget> OnGenerateTagSourcesComboBox(TSharedPtr<FString> InItem);

	/** Creates the text displayed by the combo box when an option is selected */
	FText CreateTagSourcesComboBoxContent() const;

	/** Creates the text displayed by the combo box tooltip when an option is selected */
	FText CreateTagSourcesComboBoxToolTip() const;

	EVisibility OnGetTagSourceFavoritesVisibility() const;
	FReply OnToggleTagSourceFavoriteClicked();
	const FSlateBrush* OnGetTagSourceFavoriteImage() const;

private:

	/** All potential INI files where a gameplay tag can be stored */
	TArray<TSharedPtr<FString> > TagSources;

	/** The name of the next gameplay tag to create */
	TSharedPtr<SEditableTextBox> TagNameTextBox;

	/** The comment to asign to the next gameplay tag to create */
	TSharedPtr<SEditableTextBox> TagCommentTextBox;

	/** The INI file where the next gameplay tag will be created */
	TSharedPtr<SSearchableComboBox> TagSourcesComboBox;


	/** Callback for when a new gameplay tag has been added to the INI files */
	FOnGameplayTagAdded OnGameplayTagAdded;

	/** Callback to see if the gameplay tag is valid. This should be used for any specialized rules that are not covered by IsValidGameplayTagString */
	FIsValidTag IsValidTag;

	/** Guards against the window closing when it loses focus due to source control checking out a file */
	bool bAddingNewTag = false;

	/** Tracks if this widget should get keyboard focus */
	bool bShouldGetKeyboardFocus = false;

	/** If true, this widget is displaying restricted tags; if false this widget displays regular gameplay tags. */
	bool bRestrictedTags = false;

	/** Cached from NewTagName argument. */
	FString DefaultNewName;

	/** Last error notification trying to create a new tag. */
	TSharedPtr<SNotificationItem> NotificationItem;
};
