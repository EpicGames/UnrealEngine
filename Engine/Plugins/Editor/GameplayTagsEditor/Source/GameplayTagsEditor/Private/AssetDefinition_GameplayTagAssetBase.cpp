// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_GameplayTagAssetBase.h"

#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IMainFrameModule.h"
#include "SGameplayTagPicker.h"
#include "ToolMenuSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_GameplayTagAssetBase)

#define LOCTEXT_NAMESPACE "UAssetDefinition_GameplayTagAssetBase"

void UAssetDefinition_GameplayTagAssetBase::AddGameplayTagsEditMenuExtension(FToolMenuSection& InSection, TArray<UObject*> InObjects, const FName& OwnedGameplayTagPropertyName)
{
	TArray<UObject*> Objects;
	TArray<FGameplayTagContainer> Containers;

	for (int32 ObjIdx = 0; ObjIdx < InObjects.Num(); ++ObjIdx)
	{
		if (UObject* CurObj = InObjects[ObjIdx])
		{
			const FStructProperty* Property = FindFProperty<FStructProperty>(CurObj->GetClass(), OwnedGameplayTagPropertyName);
			if (Property != nullptr)
			{
				const FGameplayTagContainer& Container = *Property->ContainerPtrToValuePtr<FGameplayTagContainer>(CurObj);
				Objects.Add(CurObj);
				Containers.Add(Container);
			}
		}
	}

	if (Containers.Num() > 0)
	{
		const TAttribute<FText> Label = LOCTEXT("GameplayTags_Edit", "Edit Gameplay Tags...");
		const TAttribute<FText> ToolTip = LOCTEXT("GameplayTags_EditToolTip", "Opens the Gameplay Tag Editor.");
		const FSlateIcon Icon = FSlateIcon();
		const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateLambda([Objects, Containers, OwnedGameplayTagPropertyName](const FToolMenuContext& InContext)
			{
				OpenGameplayTagEditor(Objects, Containers, OwnedGameplayTagPropertyName);
			});
		InSection.AddMenuEntry("GameplayTags_Edit", Label, ToolTip, Icon, UIAction);
	}
}

void UAssetDefinition_GameplayTagAssetBase::OpenGameplayTagEditor(TArray<UObject*> Objects, TArray<FGameplayTagContainer> Containers, const FName& OwnedGameplayTagPropertyName)
{
	if (!Objects.Num() || !Containers.Num())
	{
		return;
	}

	check(Objects.Num() == Containers.Num());

	for (UObject* Object : Objects)
	{
		check(IsValid(Object));
		Object->SetFlags(RF_Transactional);
	}

	FText Title;
	FText AssetName;

	const int32 NumAssets = Containers.Num();
	if (NumAssets > 1)
	{
		AssetName = FText::Format(LOCTEXT("AssetTypeActions_GameplayTagAssetBaseMultipleAssets", "{0} Assets"), FText::AsNumber(NumAssets));
		Title = FText::Format(LOCTEXT("AssetTypeActions_GameplayTagAssetBaseEditorTitle", "Tag Editor: Owned Gameplay Tags: {0}"), AssetName);
	}
	else
	{
		AssetName = FText::FromString(GetNameSafe(Objects[0]));
		Title = FText::Format(LOCTEXT("AssetTypeActions_GameplayTagAssetBaseEditorTitle", "Tag Editor: Owned Gameplay Tags: {0}"), AssetName);
	}

	TSharedPtr<SWindow> Window = SNew(SWindow)
		.Title(Title)
		.ClientSize(FVector2D(500, 600))
		[
			SNew(SGameplayTagPicker)
				.TagContainers(Containers)
				.MaxHeight(0) // unbounded
				.OnRefreshTagContainers_Lambda([Objects, OwnedGameplayTagPropertyName = OwnedGameplayTagPropertyName](SGameplayTagPicker& TagPicker)
					{
						// Refresh tags from objects, this is called e.g. on post undo/redo. 
						TArray<FGameplayTagContainer> Containers;
						for (UObject* Object : Objects)
						{
							// Adding extra entry even if the object has gone invalid to keep the container count the same as object count.
							FGameplayTagContainer& NewContainer = Containers.AddDefaulted_GetRef();
							if (IsValid(Object))
							{
								const FStructProperty* Property = FindFProperty<FStructProperty>(Object->GetClass(), OwnedGameplayTagPropertyName);
								if (Property != nullptr)
								{
									NewContainer = *Property->ContainerPtrToValuePtr<FGameplayTagContainer>(Object);
								}
							}
						}
						TagPicker.SetTagContainers(Containers);
					})
				.OnTagChanged_Lambda([Objects, OwnedGameplayTagPropertyName = OwnedGameplayTagPropertyName](const TArray<FGameplayTagContainer>& TagContainers)
					{
						// Sanity check that our arrays are in sync.
						if (Objects.Num() != TagContainers.Num())
						{
							return;
						}

						for (int32 Index = 0; Index < Objects.Num(); Index++)
						{
							UObject* Object = Objects[Index];
							if (!IsValid(Object))
							{
								continue;
							}

							FStructProperty* Property = FindFProperty<FStructProperty>(Object->GetClass(), OwnedGameplayTagPropertyName);
							if (!Property)
							{
								continue;
							}

							Object->Modify();
							FGameplayTagContainer& Container = *Property->ContainerPtrToValuePtr<FGameplayTagContainer>(Object);

							FEditPropertyChain PropertyChain;
							PropertyChain.AddHead(Property);
							Object->PreEditChange(PropertyChain);

							Container = TagContainers[Index];

							FPropertyChangedEvent PropertyEvent(Property);
							Object->PostEditChangeProperty(PropertyEvent);
						}
					})
		];

	IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
	if (MainFrameModule.GetParentWindow().IsValid())
	{
		FSlateApplication::Get().AddWindowAsNativeChild(Window.ToSharedRef(), MainFrameModule.GetParentWindow().ToSharedRef());
	}
	else
	{
		FSlateApplication::Get().AddWindow(Window.ToSharedRef());
	}
}

#undef LOCTEXT_NAMESPACE
