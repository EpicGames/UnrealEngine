// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTagContainerCustomization.h"
#include "DetailWidgetRow.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "GameplayTagsEditorModule.h"
#include "GameplayTagsManager.h"
#include "SGameplayTagContainerCombo.h"
#include "HAL/PlatformApplicationMisc.h"
#include "ScopedTransaction.h"
#include "GameplayTagEditorUtilities.h"
#include "IDetailChildrenBuilder.h"
#include "SGameplayTagPicker.h"
#include "Kismet2/BlueprintEditorUtils.h"

#define LOCTEXT_NAMESPACE "GameplayTagContainerCustomization"

TSharedRef<IPropertyTypeCustomization> FGameplayTagContainerCustomizationPublic::MakeInstance()
{
	return MakeShareable(new FGameplayTagContainerCustomization());
}

FGameplayTagContainerCustomization::FGameplayTagContainerCustomization()
	: CreatorBP(nullptr)
{
}

void FGameplayTagContainerCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	StructPropertyHandle = InStructPropertyHandle;
	FString CurrentValue;
	StructPropertyHandle->GetValueAsFormattedString(CurrentValue);

	const FString CategoriesString = UGameplayTagsManager::Get().GetCategoriesMetaFromPropertyHandle(StructPropertyHandle);
	if (!CategoriesString.IsEmpty())
	{
		TArray<FString> CategoriesStringList;

		if (CategoriesString.ParseIntoArray(CategoriesStringList, TEXT(","), true) > 0)
		{
			UGameplayTagsManager::Get().RequestGameplayTagContainer(CategoriesStringList, MetaDataCategories, false);
		}
	}
	
	FGameplayTagContainer CurrentTagContainer;
	CurrentTagContainer.FromExportString(CurrentValue);
	bool bHasValidCategories = CurrentTagContainer.IsEmpty() || MetaDataCategories.IsEmpty() || CurrentTagContainer.HasAll(MetaDataCategories);

	HeaderRow
		.FilterString(FText::AsCultureInvariant(CurrentTagContainer.ToStringSimple()))
		.NameContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(FMargin(0,2,0,1))
			.AutoWidth()
			[
				SNew(SImage)
				.Visibility(bHasValidCategories ? EVisibility::Collapsed : EVisibility::Visible)
				.Image(FAppStyle::GetBrush("Icons.WarningWithColor"))
				.ToolTipText(FText::FromString("Current tag values do not match the Gameplay Tag Roots filter."))
			]
			+ SHorizontalBox::Slot()
			[
				StructPropertyHandle->CreatePropertyNameWidget()
			]
		]
		.ValueContent()
		.VAlign(VAlign_Center)
		[
			SNew(SBox)
			.Padding(FMargin(0,2,0,1))
			[
				SNew(SGameplayTagContainerCombo)
				.PropertyHandle(StructPropertyHandle)
			]
		]
	.PasteAction(FUIAction(
	FExecuteAction::CreateSP(this, &FGameplayTagContainerCustomization::OnPasteTag),
		FCanExecuteAction::CreateSP(this, &FGameplayTagContainerCustomization::CanPasteTag)));
}

void FGameplayTagContainerCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& ChildBuilder,
	IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (StructPropertyHandle->GetProperty()->IsNative())
	{
		return;
	}

	if (const UBlueprintGeneratedClass* GeneratedClass = Cast<UBlueprintGeneratedClass>(StructPropertyHandle->GetOuterBaseClass()))
	{
		CreatorBP = Cast<UBlueprint>(GeneratedClass->ClassGeneratedBy);
		if (!CreatorBP.IsValid() || !FBlueprintEditorUtils::IsVariableCreatedByBlueprint(CreatorBP.Get(), StructPropertyHandle->GetProperty()))
		{
			return;
		}
	}

	ChildBuilder.AddCustomRow(LOCTEXT("Gameplay Tag Roots", "Categories"))
			.NameContent()
			[
				SNew(STextBlock)
					.Text(FText::FromString("Gameplay Tag Roots"))
					.ToolTipText(FText::FromString("Selects the allowed root tags this variable can inherit from"))
					.Font(StructCustomizationUtils.GetRegularFont())
			]
			.ValueContent()
			.VAlign(VAlign_Center)
			[
				SNew(SGameplayTagContainerCombo)
					.TagContainer(this, &FGameplayTagContainerCustomization::GetMetaDataCategories)
					.ToolTipText(FText::FromString("Selects the allowed root tags this variable can inherit from"))
					.OnTagContainerChanged_Lambda([this](const FGameplayTagContainer& NewTagContainer)
					{
						MetaDataCategories = NewTagContainer;
					})
					.OnTagCleared(this, &FGameplayTagContainerCustomization::OnTagCleared)
					.OnTagContainerComboClosed(this, &FGameplayTagContainerCustomization::OnGameplayTagContainerComboClosed)
					
			];
}

void FGameplayTagContainerCustomization::OnGameplayTagContainerComboClosed(const FGameplayTagContainer& NewTagContainer)
{
	MetaDataCategories = NewTagContainer;
	FString ContainerString = MetaDataCategories.ToStringSimple();
	ContainerString.ReplaceInline(TEXT(" "), TEXT(""));

	if (UBlueprint* CreatorBPPtr = CreatorBP.Get())
	{
		if (!MetaDataCategories.IsEmpty())
		{
			FBlueprintEditorUtils::SetBlueprintVariableMetaData(CreatorBPPtr, StructPropertyHandle->GetProperty()->GetFName(), StructPropertyHandle->GetProperty()->GetOwner<UFunction>(), TEXT("Categories"), ContainerString);
		}
		else
		{
			FBlueprintEditorUtils::RemoveBlueprintVariableMetaData(CreatorBPPtr, StructPropertyHandle->GetProperty()->GetFName(), StructPropertyHandle->GetProperty()->GetOwner<UFunction>(), TEXT("Categories"));
		}
	}
	else
	{
		if (!MetaDataCategories.IsEmpty())
		{
			StructPropertyHandle->GetProperty()->SetMetaData(TEXT("Categories"), *ContainerString);
		}
		else
		{
			StructPropertyHandle->GetProperty()->RemoveMetaData(TEXT("Categories"));
		}
	}
}

void FGameplayTagContainerCustomization::OnTagCleared()
{
	OnGameplayTagContainerComboClosed(MetaDataCategories);
}

void FGameplayTagContainerCustomization::OnPasteTag() const
{
	if (!StructPropertyHandle.IsValid())
	{
		return;
	}
	
	FString PastedText;
	FPlatformApplicationMisc::ClipboardPaste(PastedText);
	bool bHandled = false;

	// Try to paste single tag
	const FGameplayTag PastedTag = UE::GameplayTags::EditorUtilities::GameplayTagTryImportText(PastedText);
	if (PastedTag.IsValid())
	{
		TArray<FString> NewValues;
		SGameplayTagPicker::EnumerateEditableTagContainersFromPropertyHandle(StructPropertyHandle.ToSharedRef(), [&NewValues, PastedTag](const FGameplayTagContainer& EditableTagContainer)
		{
			FGameplayTagContainer TagContainerCopy = EditableTagContainer;
			TagContainerCopy.AddTag(PastedTag);

			NewValues.Add(TagContainerCopy.ToString());
			return true;
		});

		FScopedTransaction Transaction(LOCTEXT("GameplayTagContainerCustomization_PasteTag", "Paste Gameplay Tag"));
		StructPropertyHandle->SetPerObjectValues(NewValues);
		bHandled = true;
	}

	// Try to paste a container
	if (!bHandled)
	{
		const FGameplayTagContainer PastedTagContainer = UE::GameplayTags::EditorUtilities::GameplayTagContainerTryImportText(PastedText);
		if (PastedTagContainer.IsValid())
		{
			// From property
			FScopedTransaction Transaction(LOCTEXT("GameplayTagContainerCustomization_PasteTagContainer", "Paste Gameplay Tag Container"));
			StructPropertyHandle->SetValueFromFormattedString(PastedText);
			bHandled = true;
		}
	}
}

bool FGameplayTagContainerCustomization::CanPasteTag() const
{
	if (!StructPropertyHandle.IsValid())
	{
		return false;
	}

	FString PastedText;
	FPlatformApplicationMisc::ClipboardPaste(PastedText);
	
	const FGameplayTag PastedTag = UE::GameplayTags::EditorUtilities::GameplayTagTryImportText(PastedText);
	if (PastedTag.IsValid())
	{
		return true;
	}

	const FGameplayTagContainer PastedTagContainer = UE::GameplayTags::EditorUtilities::GameplayTagContainerTryImportText(PastedText);
	if (PastedTagContainer.IsValid())
	{
		return true;
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
