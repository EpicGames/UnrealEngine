// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "GameplayTagContainer.h"

struct FGameplayTag;
class IPropertyHandle;
class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IPropertyTypeCustomizationUtils;

/** Customization for the gameplay tag container struct */
class FGameplayTagContainerCustomization : public IPropertyTypeCustomization
{
public:
	FGameplayTagContainerCustomization();

	/** Overridden to show an edit button to launch the gameplay tag editor */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	
	/** Overridden to do nothing */
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:

	FGameplayTagContainer GetMetaDataCategories() const
	{
		return MetaDataCategories;
	}

	void OnGameplayTagContainerComboClosed(const FGameplayTagContainer& NewTagContainer);
	void OnTagCleared();
	
	/** Cached property handle */
	TSharedPtr<IPropertyHandle> StructPropertyHandle;
	FGameplayTagContainer MetaDataCategories;
	TWeakObjectPtr<UBlueprint> CreatorBP;

	void OnPasteTag() const;
	bool CanPasteTag() const;
};

