// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTagCustomization.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "PropertyHandle.h"
#include "DetailWidgetRow.h"
#include "GameplayTagsManager.h"
#include "IDetailChildrenBuilder.h"
#include "GameplayTagsEditorModule.h"
#include "SGameplayTagCombo.h"
#include "SGameplayTagContainerCombo.h"
#include "SGameplayTagPicker.h"
#include "Kismet2/BlueprintEditorUtils.h"

#define LOCTEXT_NAMESPACE "GameplayTagCustomization"

//---------------------------------------------------------
// FGameplayTagCustomizationPublic
//---------------------------------------------------------

TSharedRef<IPropertyTypeCustomization> FGameplayTagCustomizationPublic::MakeInstance()
{
	return MakeShareable(new FGameplayTagCustomization());
}

//---------------------------------------------------------
// FGameplayTagCustomization
//---------------------------------------------------------

FGameplayTagCustomization::FGameplayTagCustomization()
	: CreatorBP(nullptr)
{
}

void FGameplayTagCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	StructPropertyHandle = InStructPropertyHandle;
	FString CurrentValue;
	StructPropertyHandle->GetValueAsFormattedString(CurrentValue);

	const FString CategoriesString = UGameplayTagsManager::Get().GetCategoriesMetaFromPropertyHandle(StructPropertyHandle);
	if (!CategoriesString.IsEmpty())
	{
		TArray<FString> CategoriesStringList;

		if (CategoriesString.ParseIntoArray(CategoriesStringList, TEXT(","), true) > 0)
		{
			UGameplayTagsManager::Get().RequestGameplayTagContainer(CategoriesStringList, MetaDataCategories, false);
		}
	}

	FGameplayTag CurrentTag;
	CurrentTag.FromExportString(CurrentValue);
	bool bHasValidCategories = CurrentTag == FGameplayTag::EmptyTag || MetaDataCategories.IsEmpty() || CurrentTag.MatchesAny(MetaDataCategories);

	HeaderRow
	.FilterString(FText::AsCultureInvariant(CurrentTag.ToString()))
	.NameContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(FMargin(0,2,0,1))
		.AutoWidth()
		[
			SNew(SImage)
			.Visibility(bHasValidCategories ? EVisibility::Collapsed : EVisibility::Visible)
			.Image(FAppStyle::GetBrush("Icons.WarningWithColor"))
			.ToolTipText(FText::FromString("Current tag values do not match the Gameplay Tag Roots filter."))
		]
		+ SHorizontalBox::Slot()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
	]
	.ValueContent()
	[
		SNew(SBox)
		.Padding(FMargin(0,2,0,1))
		[
			SNew(SGameplayTagCombo)
			.PropertyHandle(StructPropertyHandle)
		]
	];
}

void FGameplayTagCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& ChildBuilder,
	IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (StructPropertyHandle->GetProperty()->IsNative())
	{
		return;
	}

	if (const UBlueprintGeneratedClass* GeneratedClass = Cast<UBlueprintGeneratedClass>(StructPropertyHandle->GetOuterBaseClass()))
	{
		CreatorBP = Cast<UBlueprint>(GeneratedClass->ClassGeneratedBy);
		if (!CreatorBP.IsValid() || !FBlueprintEditorUtils::IsVariableCreatedByBlueprint(CreatorBP.Get(), StructPropertyHandle->GetProperty()))
		{
			return;
		}
	}

	const FString CategoriesString = UGameplayTagsManager::Get().GetCategoriesMetaFromPropertyHandle(StructPropertyHandle);
	if (!CategoriesString.IsEmpty())
	{
		TArray<FString> CategoriesStringList;

		if (CategoriesString.ParseIntoArray(CategoriesStringList, TEXT(","), true) > 0)
		{
			UGameplayTagsManager::Get().RequestGameplayTagContainer(CategoriesStringList, MetaDataCategories);
		}
	}

	ChildBuilder.AddCustomRow(LOCTEXT("Gameplay Tag Roots", "Categories"))
			.NameContent()
			[
				SNew(STextBlock)
					.Text(FText::FromString("Gameplay Tag Roots"))
					.ToolTipText(FText::FromString("Selects the allowed root tags this variable can inherit from"))
					.Font(StructCustomizationUtils.GetRegularFont())
			]
			.ValueContent()
			.VAlign(VAlign_Center)
			[
				SNew(SGameplayTagContainerCombo)
					.TagContainer(this, &FGameplayTagCustomization::GetMetaDataCategories)
					.ToolTipText(FText::FromString("Selects the allowed root tags this variable can inherit from"))
					.OnTagContainerChanged_Lambda([this](const FGameplayTagContainer& NewTagContainer)
					{
						MetaDataCategories = NewTagContainer;
					})
					.OnTagCleared(this, &FGameplayTagCustomization::OnTagCleared)
					.OnTagContainerComboClosed(this, &FGameplayTagCustomization::OnGameplayTagContainerComboClosed)
					
			];
}

void FGameplayTagCustomization::OnGameplayTagContainerComboClosed(const FGameplayTagContainer& NewTagContainer)
{
	MetaDataCategories = NewTagContainer;
	FString ContainerString = MetaDataCategories.ToStringSimple();
	ContainerString.ReplaceInline(TEXT(" "), TEXT(""));

	if (UBlueprint* CreatorBPPtr = CreatorBP.Get())
	{
		if (!MetaDataCategories.IsEmpty())
		{
			FBlueprintEditorUtils::SetBlueprintVariableMetaData(CreatorBPPtr, StructPropertyHandle->GetProperty()->GetFName(), StructPropertyHandle->GetProperty()->GetOwner<UFunction>(), TEXT("Categories"), ContainerString);
		}
		else
		{
			FBlueprintEditorUtils::RemoveBlueprintVariableMetaData(CreatorBPPtr, StructPropertyHandle->GetProperty()->GetFName(), StructPropertyHandle->GetProperty()->GetOwner<UFunction>(), TEXT("Categories"));
		}
	}
	else
	{
		if (!MetaDataCategories.IsEmpty())
		{
			StructPropertyHandle->GetProperty()->SetMetaData(TEXT("Categories"), *ContainerString);
		}
		else
		{
			StructPropertyHandle->GetProperty()->RemoveMetaData(TEXT("Categories"));
		}
	}
}

void FGameplayTagCustomization::OnTagCleared()
{
	OnGameplayTagContainerComboClosed(MetaDataCategories);
}

//---------------------------------------------------------
// FGameplayTagCreationWidgetHelperDetails
//---------------------------------------------------------

TSharedRef<IPropertyTypeCustomization> FGameplayTagCreationWidgetHelperDetails::MakeInstance()
{
	return MakeShareable(new FGameplayTagCreationWidgetHelperDetails());
}

void FGameplayTagCreationWidgetHelperDetails::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	HeaderRow.WholeRowContent()[ SNullWidget::NullWidget ];
}

void FGameplayTagCreationWidgetHelperDetails::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	const FString FilterString = UGameplayTagsManager::Get().GetCategoriesMetaFromPropertyHandle(StructPropertyHandle);
	constexpr float MaxPropertyWidth = 480.0f;
	constexpr float MaxPropertyHeight = 240.0f;

	StructBuilder.AddCustomRow(NSLOCTEXT("GameplayTagReferenceHelperDetails", "NewTag", "NewTag"))
		.ValueContent()
		.MaxDesiredWidth(MaxPropertyWidth)
		[
			SAssignNew(TagWidget, SGameplayTagPicker)
			.Filter(FilterString)
			.MultiSelect(false)
			.GameplayTagPickerMode (EGameplayTagPickerMode::ManagementMode)
			.MaxHeight(MaxPropertyHeight)
		];
}

#undef LOCTEXT_NAMESPACE
