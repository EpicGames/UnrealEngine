// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "GameplayTagContainer.h"

struct EVisibility;

class IPropertyHandle;
class FDetailWidgetRow;
class IDetailChildrenBuilder;
class SGameplayTagPicker;

/** Customization for the gameplay tag struct */
class FGameplayTagCustomization : public IPropertyTypeCustomization
{
public:

	FGameplayTagCustomization();

	/** Overridden to show an edit button to launch the gameplay tag editor */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

	/** Overridden to display "Categories" metadata if created from BP */
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	
	/** Overridden to show tags and value inline */
	virtual bool ShouldInlineKey() const override { return true; }

private:
	
	/** Cached property handle */
	TSharedPtr<IPropertyHandle> StructPropertyHandle;

	FGameplayTagContainer GetMetaDataCategories() const
	{
			return MetaDataCategories;
	}

	void OnGameplayTagContainerComboClosed(const FGameplayTagContainer& NewTagContainer);
	void OnTagCleared();

	/** Edited Tag */
	FGameplayTag Tag;
	FGameplayTagContainer MetaDataCategories;
	TWeakObjectPtr<UBlueprint> CreatorBP;
};

/** Customization for FGameplayTagCreationWidgetHelper showing an add tag button */
class FGameplayTagCreationWidgetHelperDetails : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	
	TSharedPtr<SGameplayTagPicker> TagWidget;
};
