// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTagEditorUtilities.h"
#include "GameplayTagsManager.h"
#include "K2Node_CallFunction.h"
#include "K2Node_VariableSet.h"
#include "K2Node_FunctionTerminator.h"
#include "Misc/OutputDeviceNull.h"

namespace UE::GameplayTags::EditorUtilities
{

static FName NAME_Categories = FName("Categories");
static FName NAME_GameplayTagFilter = FName("GameplayTagFilter");

FString ExtractTagFilterStringFromGraphPin(UEdGraphPin* InTagPin)
{
	FString FilterString;

	if (ensure(InTagPin))
	{
		const UGameplayTagsManager& TagManager = UGameplayTagsManager::Get();
		if (UScriptStruct* PinStructType = Cast<UScriptStruct>(InTagPin->PinType.PinSubCategoryObject.Get()))
		{
			FilterString = TagManager.GetCategoriesMetaFromField(PinStructType);
		}

		UEdGraphNode* OwningNode = InTagPin->GetOwningNode();

		if (FilterString.IsEmpty())
		{
			FilterString = OwningNode->GetPinMetaData(InTagPin->PinName, NAME_Categories);
		}
		if (FilterString.IsEmpty())
		{
			FilterString = OwningNode->GetPinMetaData(InTagPin->PinName, NAME_GameplayTagFilter);
		}

		if (FilterString.IsEmpty())
		{
			if (const UK2Node_CallFunction* CallFuncNode = Cast<UK2Node_CallFunction>(OwningNode))
			{
				if (const UFunction* TargetFunction = CallFuncNode->GetTargetFunction())
				{
					FilterString = TagManager.GetCategoriesMetaFromFunction(TargetFunction, InTagPin->PinName);
				}
			}
			else if (const UK2Node_VariableSet* VariableSetNode = Cast<UK2Node_VariableSet>(OwningNode))
			{
				if (FProperty* SetVariable = VariableSetNode->GetPropertyForVariable())
				{
					FilterString = TagManager.GetCategoriesMetaFromField(SetVariable);
				}
			}
			else if (const UK2Node_FunctionTerminator* FuncTermNode = Cast<UK2Node_FunctionTerminator>(OwningNode))
			{
				if (const UFunction* SignatureFunction = FuncTermNode->FindSignatureFunction())
				{
					FilterString = TagManager.GetCategoriesMetaFromFunction(SignatureFunction, InTagPin->PinName);
				}
			}
		}
	}

	return FilterString;
}

FString GameplayTagExportText(const FGameplayTag Tag)
{
	FString ExportString;
	FGameplayTag::StaticStruct()->ExportText(ExportString, &Tag, &Tag, /*OwnerObject*/nullptr, /*PortFlags*/0, /*ExportRootScope*/nullptr);
	return ExportString;
}

FGameplayTag GameplayTagTryImportText(const FString& Text, const int32 PortFlags)
{
	FOutputDeviceNull NullOut;
	FGameplayTag Tag;
	FGameplayTag::StaticStruct()->ImportText(*Text, &Tag, /*OwnerObject*/nullptr, PortFlags, &NullOut, FGameplayTag::StaticStruct()->GetName(), /*bAllowNativeOverride*/true);
	return Tag;
}

FString GameplayTagContainerExportText(const FGameplayTagContainer& TagContainer)
{
	FString ExportString;
	FGameplayTagContainer::StaticStruct()->ExportText(ExportString, &TagContainer, &TagContainer, /*OwnerObject*/nullptr, /*PortFlags*/0, /*ExportRootScope*/nullptr);
	return ExportString;
}

FGameplayTagContainer GameplayTagContainerTryImportText(const FString& Text, const int32 PortFlags)
{
	FOutputDeviceNull NullOut;
	FGameplayTagContainer TagContainer;
	FGameplayTagContainer::StaticStruct()->ImportText(*Text, &TagContainer, /*OwnerObject*/nullptr, PortFlags, &NullOut, FGameplayTagContainer::StaticStruct()->GetName(), /*bAllowNativeOverride*/true);
	return TagContainer;
}

FString GameplayTagQueryExportText(const FGameplayTagQuery& TagQuery)
{
	FString ExportString;
	FGameplayTagQuery::StaticStruct()->ExportText(ExportString, &TagQuery, &TagQuery, /*OwnerObject*/nullptr, /*PortFlags*/0, /*ExportRootScope*/nullptr);
	return ExportString;
}
	
FGameplayTagQuery GameplayTagQueryTryImportText(const FString Text, int32 PortFlags)
{
	FOutputDeviceNull NullOut;
	FGameplayTagQuery TagQuery;
	FGameplayTagQuery::StaticStruct()->ImportText(*Text, &TagQuery, /*OwnerObject*/nullptr, PortFlags, &NullOut, FGameplayTagQuery::StaticStruct()->GetName(), /*bAllowNativeOverride*/true);
	return TagQuery;
}

FString FormatGameplayTagQueryDescriptionToLines(const FString& Desc)
{
	// Reformat automatic description.
	if (Desc.StartsWith(TEXT(" ALL("))
		|| Desc.StartsWith(TEXT(" ANY("))
		|| Desc.StartsWith(TEXT(" NONE(")))
	{
		TStringBuilder<1024> String;

		auto OutputIndent = [](FStringBuilderBase& Out, const int32 Indent)
		{
			Out += TEXT("\n");
			for (int32 Idx = 0; Idx < Indent; Idx++)
			{
				Out += TEXT("    ");
			}
		};

		int32 Indent = 0;
		for (const TCHAR Char : Desc)
		{
			if (Char == TEXT(' '))
			{
				// Skip white space
			}
			else if (Char == TEXT('('))
			{
				String += Char;
				Indent++;
				OutputIndent(String, Indent);
			}
			else if (Char == TEXT(')'))
			{
				Indent--;
				OutputIndent(String, Indent);
				String += Char;
			}
			else if (Char == TEXT(','))
			{
				String += Char;
				OutputIndent(String, Indent);
			}
			else
			{
				String += Char;
			}
		}

		return String.ToString();
	}

	return Desc;
}


};