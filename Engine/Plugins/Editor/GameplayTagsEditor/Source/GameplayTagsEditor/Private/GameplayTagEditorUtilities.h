// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "GameplayTagContainer.h"

// Forward declarations
class UEdGraphPin;
class FString;

/** Utility functions for gameplay tag-related pins */
namespace UE::GameplayTags::EditorUtilities
{
	/**
	 * Given a editor graph pin representing a tag or container, extract the appropriate filter
	 * string from any associated metadata
	 * 
	 * @param InTagPin	Pin to extract from
	 * 
	 * @return Filter string, if any, to apply from the tag pin based on metadata
	 */
	FString ExtractTagFilterStringFromGraphPin(UEdGraphPin* InTagPin);

	/**
	 * Exports a gameplay tag to text compatible with Import/Export text.
	 * @param Tag tag to export
	 * @return tag as text.
	 */
	FString GameplayTagExportText(const FGameplayTag Tag);

	/**
     * Tries to import gameplay tag from text.
	 * @param Text string to import from
	 * @param PortFlags EPropertyPortFlags controlling import behavior. 
	 * @return parsed gameplay tag, or None if import failed. 
     */
	FGameplayTag GameplayTagTryImportText(const FString& Text, const int32 PortFlags = 0);

	/**
	 * Exports a gameplay tag container to text compatible with Import/Export text.
	 * @param TagContainer tag container to export
	 * @return tag container as text.
	 */
	FString GameplayTagContainerExportText(const FGameplayTagContainer& TagContainer);

	/**
     * Tries to import gameplay tag container from text.
	 * @param Text string to import from
	 * @param PortFlags EPropertyPortFlags controlling import behavior.
	 * @return parsed gameplay tag container, or empty if import failed. 
     */
	FGameplayTagContainer GameplayTagContainerTryImportText(const FString& Text, int32 PortFlags = 0);

	/**
	 * Exports a gameplay tag query to text compatible with Import/Export text.
	 * @param TagQuery tag query to export
	 * @return tag query as text.
	 */
	FString GameplayTagQueryExportText(const FGameplayTagQuery& TagQuery);
	
	/**
	 * Tries to import gameplay tag query from text.
	 * @param Text string to import from
	 * @param PortFlags EPropertyPortFlags controlling import behavior.
	 * @return parsed gameplay tag query, or empty if import failed. 
	 */
	FGameplayTagQuery GameplayTagQueryTryImportText(const FString Text, int32 PortFlags = 0);

	/**
	 * Formats Gameplay Tag Query description to multiple lines.
	 * @param Desc Description to format
	 * @return Description in multiline format.
	 */
	FString FormatGameplayTagQueryDescriptionToLines(const FString& Desc);

} // UE::GameplayTags::EditorUtilities
