// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTagQueryCustomization.h"
#include "GameplayTagContainer.h"
#include "DetailWidgetRow.h"
#include "IPropertyTypeCustomization.h"
#include "SGameplayTagQueryEntryBox.h"

void FGameplayTagQueryCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	HeaderRow
		.NameContent()
		[
			InStructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.VAlign(VAlign_Center)
		[
			SNew(SGameplayTagQueryEntryBox)
			.DescriptionMaxWidth(400.0f)
			.PropertyHandle(InStructPropertyHandle)
		];
}
