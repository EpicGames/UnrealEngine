// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

struct EVisibility;
class IPropertyHandle;
class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IPropertyTypeCustomizationUtils;

/** Customization for the gameplay tag query struct */
class FGameplayTagQueryCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FGameplayTagQueryCustomization);
	}

	/** Overridden to show an edit button to launch the gameplay tag editor */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {}

private:
};

