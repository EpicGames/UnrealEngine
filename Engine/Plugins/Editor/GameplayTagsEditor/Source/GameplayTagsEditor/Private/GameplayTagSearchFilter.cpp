// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTagSearchFilter.h"
#include "Engine/Blueprint.h"
#include "FrontendFilterBase.h"
#include "Misc/ConfigCacheIni.h"
#include "Widgets/SBoxPanel.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#include "GameplayTagsManager.h"
#include "SGameplayTagPicker.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayTagSearchFilter)


#define LOCTEXT_NAMESPACE "GameplayTagSearchFilter"

//////////////////////////////////////////////////////////////////////////
//

/** A filter that search for assets using a specific gameplay tag */
class FFrontendFilter_GameplayTags : public FFrontendFilter
{
public:
	FFrontendFilter_GameplayTags(TSharedPtr<FFrontendFilterCategory> InCategory)
		: FFrontendFilter(InCategory)
	{
	}

	// FFrontendFilter implementation
	virtual FLinearColor GetColor() const override { return FLinearColor::Red; }
	virtual FString GetName() const override { return TEXT("GameplayTagFilter"); }
	virtual FText GetDisplayName() const override;
	virtual FText GetToolTipText() const override;
	virtual void ModifyContextMenu(FMenuBuilder& MenuBuilder) override;
	virtual void SaveSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) const override;
	virtual void LoadSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) override;
	// End of FFrontendFilter implementation

	// IFilter implementation
	virtual bool PassesFilter(FAssetFilterType InItem) const override;
	// End of IFilter implementation

protected:
	// Container of selected search tags (the asset is shown if *any* of these match)
	FGameplayTagContainer TagContainer;

	bool ProcessStruct(void* Data, UStruct* Struct) const;

	bool ProcessProperty(void* Data, FProperty* Prop) const;

	void OnTagWidgetChanged(const TArray<FGameplayTagContainer>& TagContainers);
};

void FFrontendFilter_GameplayTags::ModifyContextMenu(FMenuBuilder& MenuBuilder)
{
	FUIAction Action;

	MenuBuilder.BeginSection(TEXT("ComparsionSection"), LOCTEXT("ComparisonSectionHeading", "Gameplay Tag(s) to search for"));

	TArray<FGameplayTagContainer> EditableContainers;
	EditableContainers.Add(TagContainer);

	TSharedRef<SWidget> TagWidget =
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.MaxHeight(300)
		[
			SNew(SGameplayTagPicker)
			.TagContainers(EditableContainers)
			.MultiSelect(true)
			.OnTagChanged_Raw(this, &FFrontendFilter_GameplayTags::OnTagWidgetChanged)
		];
 	MenuBuilder.AddWidget(TagWidget, FText::GetEmpty(), /*bNoIndent=*/ false);
}

FText FFrontendFilter_GameplayTags::GetDisplayName() const
{
	if (TagContainer.Num() == 0)
	{
		return LOCTEXT("AnyGameplayTagDisplayName", "Gameplay Tags");
	}
	else
	{
		FString QueryString;

		int32 Count = 0;
		for (const FGameplayTag& Tag : TagContainer)
		{
			if (Count > 0)
			{
				QueryString += TEXT(" | ");
			}

			QueryString += Tag.ToString();
			++Count;
		}


		return FText::Format(LOCTEXT("GameplayTagListDisplayName", "Gameplay Tags ({0})"), FText::AsCultureInvariant(QueryString));
	}
}

FText FFrontendFilter_GameplayTags::GetToolTipText() const
{
	if (TagContainer.Num() == 0)
	{
		return LOCTEXT("AnyGameplayTagFilterDisplayTooltip", "Search for any *loaded* Blueprint or asset that contains a gameplay tag (right-click to choose tags).");
	}
	else
	{
		return LOCTEXT("GameplayTagFilterDisplayTooltip", "Search for any *loaded* Blueprint or asset that has a gameplay tag which matches any of the selected tags (right-click to choose tags).");
	}
}

void FFrontendFilter_GameplayTags::SaveSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) const
{
	TArray<FString> TagStrings;
	TagStrings.Reserve(TagContainer.Num());
	for (const FGameplayTag& Tag : TagContainer)
	{
		TagStrings.Add(Tag.GetTagName().ToString());
	}

	GConfig->SetArray(*IniSection, *(SettingsString + TEXT(".Tags")), TagStrings, IniFilename);
}

void FFrontendFilter_GameplayTags::LoadSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString)
{
	UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

	TArray<FString> TagStrings;
	GConfig->GetArray(*IniSection, *(SettingsString + TEXT(".Tags")), /*out*/ TagStrings, IniFilename);

	TagContainer.Reset();
	for (const FString& TagString : TagStrings)
	{
		FGameplayTag NewTag = Manager.RequestGameplayTag(*TagString, /*bErrorIfNotFound=*/ false);
		if (NewTag.IsValid())
		{
			TagContainer.AddTag(NewTag);
		}
	}
}

void FFrontendFilter_GameplayTags::OnTagWidgetChanged(const TArray<FGameplayTagContainer>& TagContainers)
{
	if (TagContainers.Num() > 0)
	{
		TagContainer = TagContainers[0];
	}
	
	BroadcastChangedEvent();
}

bool FFrontendFilter_GameplayTags::ProcessStruct(void* Data, UStruct* Struct) const
{
	for (TFieldIterator<FProperty> PropIt(Struct, EFieldIteratorFlags::IncludeSuper); PropIt; ++PropIt)
	{
		FProperty* Prop = *PropIt;

		if (ProcessProperty(Data, Prop))
		{
			return true;
		}
	}

	return false;
}

bool FFrontendFilter_GameplayTags::ProcessProperty(void* Data, FProperty* Prop) const
{
	void* InnerData = Prop->ContainerPtrToValuePtr<void>(Data);

	if (FStructProperty* StructProperty = CastField<FStructProperty>(Prop))
	{
		if (StructProperty->Struct == FGameplayTag::StaticStruct())
		{
			FGameplayTag& ThisTag = *static_cast<FGameplayTag*>(InnerData);

			const bool bAnyTagIsOK = TagContainer.Num() == 0;
			const bool bPassesTagSearch = bAnyTagIsOK || ThisTag.MatchesAny(TagContainer);

			return bPassesTagSearch;
		}
		else
		{
			return ProcessStruct(InnerData, StructProperty->Struct);
		}
	}
	else if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Prop))
	{
		FScriptArrayHelper ArrayHelper(ArrayProperty, InnerData);
		for (int32 ArrayIndex = 0; ArrayIndex < ArrayHelper.Num(); ++ArrayIndex)
		{
			void* ArrayData = ArrayHelper.GetRawPtr(ArrayIndex);

			if (ProcessProperty(ArrayData, ArrayProperty->Inner))
			{
				return true;
			}
		}
	}

	return false;
}

bool FFrontendFilter_GameplayTags::PassesFilter(FAssetFilterType InItem) const
{
	FAssetData ItemAssetData;
	if (InItem.Legacy_TryGetAssetData(ItemAssetData))
	{
		if (UObject* Object = ItemAssetData.FastGetAsset(false))
		{
			if (UBlueprint* Blueprint = Cast<UBlueprint>(Object))
			{
				return ProcessStruct(Blueprint->GeneratedClass->GetDefaultObject(), Blueprint->GeneratedClass);

				//@TODO: Check blueprint bytecode!
			}
			else if (UClass* Class = Cast<UClass>(Object))
			{
				return ProcessStruct(Class->GetDefaultObject(), Class);
			}
			else
			{
				return ProcessStruct(Object, Object->GetClass());
			}
		}
	}
	return false;
}

//////////////////////////////////////////////////////////////////////////
//

void UGameplayTagSearchFilter::AddFrontEndFilterExtensions(TSharedPtr<FFrontendFilterCategory> DefaultCategory, TArray< TSharedRef<class FFrontendFilter> >& InOutFilterList) const
{
	InOutFilterList.Add(MakeShareable(new FFrontendFilter_GameplayTags(DefaultCategory)));
}

#undef LOCTEXT_NAMESPACE

