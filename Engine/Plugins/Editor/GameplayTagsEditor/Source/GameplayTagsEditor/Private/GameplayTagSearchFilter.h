// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContentBrowserFrontEndFilterExtension.h"
#include "GameplayTagSearchFilter.generated.h"

UCLASS()
class UGameplayTagSearchFilter : public UContentBrowserFrontEndFilterExtension
{
public:
	GENERATED_BODY()

	// UContentBrowserFrontEndFilterExtension interface
	virtual void AddFrontEndFilterExtensions(TSharedPtr<class FFrontendFilterCategory> DefaultCategory, TArray< TSharedRef<class FFrontendFilter> >& InOutFilterList) const override;
	// End of UContentBrowserFrontEndFilterExtension interface
};
