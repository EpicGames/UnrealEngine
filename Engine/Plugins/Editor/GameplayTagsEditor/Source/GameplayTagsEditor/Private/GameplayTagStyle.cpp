// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTagStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateStyle.h"
#include "Styling/AppStyle.h"
#include "Components/Widget.h"
#include "Styling/StyleColors.h"

class FGameplayTagStyleSet final : public FSlateStyleSet
{
public:
	static FString InContent(const FString& RelativePath, const ANSICHAR* Extension)
	{
		static FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("MassEntity"))->GetContentDir() / TEXT("Slate");
		return (ContentDir / RelativePath) + Extension;
	}

	FGameplayTagStyleSet(const FName& InStyleSetName)
		: FSlateStyleSet(InStyleSetName)
	{
		SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
		SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

		const FTextBlockStyle& NormalText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");
		
		// Tag Chip
		const FLinearColor ChipColor = FStyleColors::Hover.GetSpecifiedColor();
		const FLinearColor ChipColorHover = FStyleColors::Hover2.GetSpecifiedColor();
		const FLinearColor ChipColorDisable = ChipColor.CopyWithNewOpacity(0.35f);
		
		const FLinearColor HollowChipColor = ChipColor.CopyWithNewOpacity(0.0);
		const FLinearColor HollowChipColorHover = ChipColorHover.CopyWithNewOpacity(0.15f);
		const FLinearColor HollowChipColorDisable = HollowChipColor.CopyWithNewOpacity(0.0f);

		FButtonStyle ChipButtonSelected = FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Button");
		ChipButtonSelected
			.SetNormal(FSlateRoundedBoxBrush(ChipColor, 3.0f))
			.SetHovered(FSlateRoundedBoxBrush(ChipColorHover, 3.0f))
			.SetPressed(FSlateRoundedBoxBrush(ChipColorHover, 3.0f))
			.SetDisabled(FSlateRoundedBoxBrush(ChipColorDisable, 3.0f))
			.SetNormalForeground(FStyleColors::ForegroundHover)
			.SetHoveredForeground(FStyleColors::White)
			.SetPressedForeground(FStyleColors::White)
			.SetDisabledForeground(FStyleColors::ForegroundHover)
			.SetNormalPadding(FMargin(5,2,2,2))
			.SetPressedPadding(FMargin(5,3,2,1));
		Set("GameplayTags.ChipButton.Selected", ChipButtonSelected);

		FButtonStyle ChipButtonUnselected = FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Button");
		ChipButtonUnselected
			.SetNormal(FSlateRoundedBoxBrush(HollowChipColor, 3.0f, ChipColor, 1.0f))
			.SetHovered(FSlateRoundedBoxBrush(HollowChipColorHover, 3.0f, ChipColorHover, 1.0f))
			.SetPressed(FSlateRoundedBoxBrush(HollowChipColorHover, 3.0f, ChipColorHover, 1.0f))
			.SetDisabled(FSlateRoundedBoxBrush(HollowChipColorDisable, 3.0f, ChipColorDisable, 1.0f))
			.SetNormalForeground(FStyleColors::ForegroundHover)
			.SetHoveredForeground(FStyleColors::White)
			.SetPressedForeground(FStyleColors::White)
			.SetDisabledForeground(FStyleColors::ForegroundHover)
			.SetNormalPadding(FMargin(5,2,5,2))
			.SetPressedPadding(FMargin(5,3,5,1));
		Set("GameplayTags.ChipButton.Unselected", ChipButtonUnselected);

		FButtonStyle ChipClearButton = FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Button");
		ChipClearButton.SetNormal(FSlateNoResource())
			.SetHovered(FSlateNoResource())
			.SetPressed(FSlateNoResource())
			.SetDisabled(FSlateNoResource())
			.SetNormalForeground(FStyleColors::ForegroundHeader)
			.SetHoveredForeground(FStyleColors::ForegroundHover)
			.SetPressedForeground(FStyleColors::ForegroundHover)
			.SetDisabledForeground(FStyleColors::Foreground)
			.SetNormalPadding(FMargin(2,0,2,0))
			.SetPressedPadding(FMargin(2,0,2,0));
		Set("GameplayTags.ChipClearButton", ChipClearButton);

		Set("GameplayTags.Container", new FSlateRoundedBoxBrush(FStyleColors::Input, 4.0f, FStyleColors::InputOutline, 1.0f));

		// Combo button with down arrow aligned to first row.
		FButtonStyle TagButton = FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton");
		TagButton.SetNormalPadding(FMargin(0,0,2,0))
			.SetPressedPadding(FMargin(0,0,2,0));
		FComboButtonStyle TagComboButton = FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("ComboButton");
		TagComboButton.SetButtonStyle(TagButton)
			.SetDownArrowPadding(FMargin(0,2,0,0))
			.SetDownArrowAlignment(VAlign_Top);
		Set("GameplayTags.ComboButton", TagComboButton);

		// Combo button with down arrow aligned to first row.
		FButtonStyle TagContainerButton = FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton");
		TagContainerButton.SetNormalPadding(FMargin(0,0,2,0))
			.SetPressedPadding(FMargin(0,0,2,0));
		FComboButtonStyle TagContainerComboButton = FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("ComboButton");
		TagContainerComboButton.SetButtonStyle(TagContainerButton)
			.SetDownArrowPadding(FMargin(0,5,0,0))
			.SetDownArrowAlignment(VAlign_Top);
		Set("GameplayTagsContainer.ComboButton", TagContainerComboButton);
	}
};


TSharedPtr<FSlateStyleSet> FGameplayTagStyle::StyleSet = nullptr;


void FGameplayTagStyle::Initialize()
{
	if (StyleSet.IsValid())
	{
		return;
	}

	StyleSet = MakeShared<FGameplayTagStyleSet>(GetStyleSetName());
	
	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
}

void FGameplayTagStyle::Shutdown()
{
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		ensure(StyleSet.IsUnique());
		StyleSet.Reset();
	}
}

FName FGameplayTagStyle::GetStyleSetName()
{
	static FName StyleName("GameplayTagStyle");
	return StyleName;
}
