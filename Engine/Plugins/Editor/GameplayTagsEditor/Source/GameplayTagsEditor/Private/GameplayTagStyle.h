// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FGameplayTagStyle
{
public:
	static void Initialize();
	static void Shutdown();
	static ISlateStyle& Get() { return *StyleSet.Get(); }
	static FName GetStyleSetName();

	static const FSlateBrush* GetBrush(FName PropertyName, const ANSICHAR* Specifier = nullptr)
	{
		return StyleSet->GetBrush(PropertyName, Specifier);
	}

	static const FLinearColor& GetColor(FName PropertyName, const ANSICHAR* Specifier = nullptr)
	{
		return StyleSet->GetColor(PropertyName, Specifier);
	}

private:
	static TSharedPtr<FSlateStyleSet> StyleSet;
};
