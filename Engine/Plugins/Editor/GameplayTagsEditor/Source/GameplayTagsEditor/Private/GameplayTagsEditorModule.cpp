// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTagsEditorModule.h"

#include "GameplayTagRedirectors.h"
#include "PropertyEditorModule.h"
#include "Factories/Factory.h"
#include "GameplayTagsManager.h"
#include "GameplayTagsGraphPanelPinFactory.h"
#include "GameplayTagsGraphPanelNodeFactory.h"
#include "GameplayTagQueryCustomization.h"
#include "GameplayTagCustomization.h"
#include "GameplayTagsSettings.h"
#include "GameplayTagsSettingsCustomization.h"
#include "GameplayTagsModule.h"
#include "HAL/FileManager.h"
#include "ISettingsModule.h"
#include "ISettingsEditorModule.h"
#include "Misc/CoreDelegates.h"
#include "Misc/ScopedSlowTask.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Editor.h"
#include "ISourceControlModule.h"
#include "SourceControlHelpers.h"
#include "HAL/PlatformFileManager.h"
#include "Stats/StatsMisc.h"
#include "Subsystems/ImportSubsystem.h"
#include "UObject/ObjectSaveContext.h"
#include "GameplayTagStyle.h"
#include "SGameplayTagPicker.h"
#include "Framework/Docking/TabManager.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "GameplayTagEditor"


namespace GameplayTagEditorModule
{
	static const FName GameplayTagManagerApp = FName("GameplayTagManagerApp");
}

class FGameplayTagsEditorModule
	: public IGameplayTagsEditorModule
{
public:

	// IModuleInterface

	virtual void StartupModule() override
	{

		auto OnCreateLambda = [](const FSpawnTabArgs& Args)
			{
				TSharedRef<SDockTab> DockTab = SNew(SDockTab)
					.Clipping(EWidgetClipping::ClipToBounds)
					.Label(LOCTEXT("GameplayTagPicker_ManagerTitle", "Gameplay Tag Manager"));

				FGameplayTagManagerWindowArgs TagManagerWindowArgs;
				TagManagerWindowArgs.bRestrictedTags = false;
				DockTab->SetContent( UE::GameplayTags::Editor::Create(TagManagerWindowArgs) );
				return DockTab;
			};
		
		FGlobalTabmanager::Get()->RegisterNomadTabSpawner(GameplayTagEditorModule::GameplayTagManagerApp, FOnSpawnTab::CreateLambda(OnCreateLambda))
			.SetDisplayName(LOCTEXT("GameplayTagPicker_ManagerTitle", "Gameplay Tag Manager"))
			.SetTooltipText(LOCTEXT("GameplayTagPicker_ManagerTitle", "Gameplay Tag Manager"))
			.SetGroup(WorkspaceMenu::GetMenuStructure().GetToolsCategory())
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.UserDefinedStruct"));

		FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FGameplayTagsEditorModule::OnPostEngineInit);
		FGameplayTagStyle::Initialize();
	}
	
	void OnPostEngineInit()
	{
		// Register the details customizer
		{
			FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
			PropertyModule.RegisterCustomPropertyTypeLayout("GameplayTagContainer", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FGameplayTagContainerCustomizationPublic::MakeInstance));
			PropertyModule.RegisterCustomPropertyTypeLayout("GameplayTag", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FGameplayTagCustomizationPublic::MakeInstance));
			PropertyModule.RegisterCustomPropertyTypeLayout("GameplayTagQuery", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FGameplayTagQueryCustomization::MakeInstance));
			PropertyModule.RegisterCustomPropertyTypeLayout("GameplayTagCreationWidgetHelper", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FGameplayTagCreationWidgetHelperDetails::MakeInstance));

			PropertyModule.RegisterCustomClassLayout(UGameplayTagsSettings::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FGameplayTagsSettingsCustomization::MakeInstance));

			PropertyModule.NotifyCustomizationModuleChanged();
		}

		TSharedPtr<FGameplayTagsGraphPanelPinFactory> GameplayTagsGraphPanelPinFactory = MakeShareable(new FGameplayTagsGraphPanelPinFactory());
		FEdGraphUtilities::RegisterVisualPinFactory(GameplayTagsGraphPanelPinFactory);

		TSharedPtr<FGameplayTagsGraphPanelNodeFactory> GameplayTagsGraphPanelNodeFactory = MakeShareable(new FGameplayTagsGraphPanelNodeFactory());
		FEdGraphUtilities::RegisterVisualNodeFactory(GameplayTagsGraphPanelNodeFactory);

		// These objects are not UDeveloperSettings because we only want them to register if the editor plugin is enabled

		if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
		{
			SettingsModule->RegisterSettings("Project", "Project", "GameplayTags",
				LOCTEXT("GameplayTagSettingsName", "GameplayTags"),
				LOCTEXT("GameplayTagSettingsNameDesc", "GameplayTag Settings"),
				GetMutableDefault<UGameplayTagsSettings>()
			);
		}

		GameplayTagPackageName = FGameplayTag::StaticStruct()->GetOutermost()->GetFName();
		GameplayTagStructName = FGameplayTag::StaticStruct()->GetFName();

		// Hook into notifications for object re-imports so that the gameplay tag tree can be reconstructed if the table changes
		if (GIsEditor)
		{
			GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetPostImport.AddRaw(this, &FGameplayTagsEditorModule::OnObjectReimported);
			FEditorDelegates::OnEditAssetIdentifiers.AddRaw(this, &FGameplayTagsEditorModule::OnEditGameplayTag);
			IGameplayTagsModule::OnTagSettingsChanged.AddRaw(this, &FGameplayTagsEditorModule::OnEditorSettingsChanged);
			UPackage::PackageSavedWithContextEvent.AddRaw(this, &FGameplayTagsEditorModule::OnPackageSaved);
		}
	}

	void OnObjectReimported(UFactory* ImportFactory, UObject* InObject)
	{
		UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

		// Re-construct the gameplay tag tree if the base table is re-imported
		if (GIsEditor && !IsRunningCommandlet() && InObject && Manager.GameplayTagTables.Contains(Cast<UDataTable>(InObject)))
		{
			Manager.EditorRefreshGameplayTagTree();
		}
	}

	virtual void ShutdownModule() override
	{
		FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);

		// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
		// we call this function before unloading the module.

		if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
		{
			SettingsModule->UnregisterSettings("Project", "Project", "GameplayTags");
			SettingsModule->UnregisterSettings("Project", "Project", "GameplayTags Developer");
		}

		if (GEditor)
		{
			GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetPostImport.RemoveAll(this);
		}
		FEditorDelegates::OnEditAssetIdentifiers.RemoveAll(this);
		IGameplayTagsModule::OnTagSettingsChanged.RemoveAll(this);
		UPackage::PackageSavedWithContextEvent.RemoveAll(this);
	}

	void OnEditorSettingsChanged()
	{
		// This is needed to make networking changes as well, so let's always refresh
		UGameplayTagsManager::Get().EditorRefreshGameplayTagTree();

		// Attempt to migrate the settings if needed
		MigrateSettings();
	}

	void WarnAboutRestart()
	{
		ISettingsEditorModule* SettingsEditorModule = FModuleManager::GetModulePtr<ISettingsEditorModule>("SettingsEditor");
		if (SettingsEditorModule)
		{
			SettingsEditorModule->OnApplicationRestartRequired();
		}
	}

	void OnPackageSaved(const FString& PackageFileName, UPackage* Package, FObjectPostSaveContext ObjectSaveContext)
	{
		if (GIsEditor && !ObjectSaveContext.IsProceduralSave())
		{
			UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

			bool bRefreshGameplayTagTree = false;

			TArray<UObject*> Objects;
			GetObjectsWithPackage(Package, Objects, EGetObjectsFlags::None);
			for (UObject* Entry : Objects)
			{
				if (UDataTable* DataTable = Cast<UDataTable>(Entry))
				{
					if (Manager.GameplayTagTables.Contains(DataTable))
					{
						bRefreshGameplayTagTree = true;
						break;
					}
				}
			}

			// Re-construct the gameplay tag tree if a data table is saved (presumably with modifications).
			if (bRefreshGameplayTagTree)
			{
				Manager.EditorRefreshGameplayTagTree();
			}
		}
	}

	void OnEditGameplayTag(TArray<FAssetIdentifier> AssetIdentifierList)
	{
		// If any of these are gameplay tags, open up tag viewer
		for (FAssetIdentifier Identifier : AssetIdentifierList)
		{
			if (Identifier.IsValue() && Identifier.PackageName == GameplayTagPackageName && Identifier.ObjectName == GameplayTagStructName)
			{
				if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
				{
					// TODO: Select tag maybe?
					SettingsModule->ShowViewer("Project", "Project", "GameplayTags");
				}
				return;
			}
		}
	}

	void ShowNotification(const FText& TextToDisplay, float TimeToDisplay, bool bLogError = false, bool bOnlyLog = false)
	{
		if (bOnlyLog)
		{
			if (bLogError)
			{
				UE_LOGF(LogGameplayTags, Error, "%ls", *TextToDisplay.ToString())
			}
			else
			{
				UE_LOGF(LogGameplayTags, Display, "%ls", *TextToDisplay.ToString())
			}
		}
		else
		{
			FNotificationInfo Info(TextToDisplay);
			Info.ExpireDuration = TimeToDisplay;

			FSlateNotificationManager::Get().AddNotification(Info);

			// Also log if error
			if (bLogError)
			{
				UE_LOGF(LogGameplayTags, Error, "%ls", *TextToDisplay.ToString())
			}
		}
	}

	void MigrateSettings()
	{
		UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

		FString DefaultEnginePath = FString::Printf(TEXT("%sDefaultEngine.ini"), *FPaths::SourceConfigDir());

		UGameplayTagsSettings* Settings = GetMutableDefault<UGameplayTagsSettings>();
		
		// The refresh has already set the in memory version of this to be correct, just need to save it out now
		if (!GConfig->GetSection(TEXT("GameplayTags"), false, DefaultEnginePath))
		{
			// Already migrated or no data
			return;
		}

		// Check out defaultengine
		GameplayTagsUpdateSourceControl(DefaultEnginePath);

		// Delete gameplay tags section entirely. This modifies the disk version
		GConfig->EmptySection(TEXT("GameplayTags"), DefaultEnginePath);

		// Remove any redirects
		GConfig->RemoveKeyFromSection(TEXT("/Script/Engine.Engine"), "+GameplayTagRedirects", DefaultEnginePath);

		// This will remove comments, etc. It is expected for someone to diff this before checking in to manually fix it
		GConfig->Flush(false, DefaultEnginePath);

		// Write out gameplaytags.ini
		GameplayTagsUpdateSourceControl(Settings->GetDefaultConfigFilename());
		Settings->TryUpdateDefaultConfigFile();

		GConfig->LoadFile(Settings->GetDefaultConfigFilename());
		
		// Write out all other tag lists
		TArray<const FGameplayTagSource*> Sources;

		Manager.FindTagSourcesWithType(EGameplayTagSourceType::TagList, Sources);
		Manager.FindTagSourcesWithType(EGameplayTagSourceType::RestrictedTagList, Sources);

		for (const FGameplayTagSource* Source : Sources)
		{
			UGameplayTagsList* TagList = Source->SourceTagList;
			if (TagList)
			{
				GameplayTagsUpdateSourceControl(TagList->ConfigFileName);
				TagList->TryUpdateDefaultConfigFile(TagList->ConfigFileName);

				// Reload off disk
				GConfig->LoadFile(TagList->ConfigFileName);

				// Explicitly remove user tags section
				GConfig->EmptySection(TEXT("UserTags"), TagList->ConfigFileName);
			}
		}

		ShowNotification(LOCTEXT("MigrationText", "Migrated Tag Settings, check DefaultEngine.ini before checking in!"), 10.0f);
	}

	void GameplayTagsUpdateSourceControl(const FString& RelativeConfigFilePath, bool bOnlyLog = false)
	{
		TArray<FString> RelativeConfigFilePaths = { RelativeConfigFilePath };
		GameplayTagsUpdateSourceControl(RelativeConfigFilePaths, bOnlyLog);
	}

	void GameplayTagsUpdateSourceControl(const TArray<FString>& RelativeConfigFilePaths, bool bOnlyLog = false)
	{
		TArray<FString> ExistingConfigPaths;
		for (const FString& RelativeConfigFilePath : RelativeConfigFilePaths)
		{
			FString ConfigPath = FPaths::ConvertRelativePathToFull(RelativeConfigFilePath);

			if (FPlatformFileManager::Get().GetPlatformFile().FileExists(*ConfigPath))
			{
				ExistingConfigPaths.Add(ConfigPath);
			}
		}

		if (ISourceControlModule::Get().IsEnabled())
		{
			FText ErrorMessage;

			if (ExistingConfigPaths.Num() == 1)
			{
				const FString& ConfigPath = ExistingConfigPaths[0];
				if (!SourceControlHelpers::CheckoutOrMarkForAdd(ConfigPath, FText::FromString(ConfigPath), NULL, ErrorMessage))
				{
					ShowNotification(ErrorMessage, 3.0f, false, bOnlyLog);
				}
			}
			else
			{
				SourceControlHelpers::CheckOutOrAddFiles(ExistingConfigPaths);
			}
		}
		else
		{
			for (const FString& ConfigPath : ExistingConfigPaths)
			{
				if (!FPlatformFileManager::Get().GetPlatformFile().SetReadOnly(*ConfigPath, false))
				{
					ShowNotification(FText::Format(LOCTEXT("FailedToMakeWritable", "Could not make {0} writable."), FText::FromString(ConfigPath)), 3.0f, false, bOnlyLog);
				}
			}
		}
	}

	bool DeleteTagRedirector(const FName& TagToDelete, bool bOnlyLog = false, bool bRefresh = true, TMap<UObject*, FString>* OutObjectsToUpdateConfig = nullptr)
	{
		UGameplayTagsSettings* Settings = GetMutableDefault<UGameplayTagsSettings>();
		UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

		for (int32 i = 0; i < Settings->GameplayTagRedirects.Num(); i++)
		{
			if (Settings->GameplayTagRedirects[i].OldTagName == TagToDelete)
			{
				Settings->GameplayTagRedirects.RemoveAt(i);

				if (bRefresh)
				{
					GameplayTagsUpdateSourceControl(Settings->GetDefaultConfigFilename());
					Settings->TryUpdateDefaultConfigFile();
					GConfig->LoadFile(Settings->GetDefaultConfigFilename());

					Manager.EditorRefreshGameplayTagTree();
				}
				else
				{
					if (OutObjectsToUpdateConfig)
					{
						OutObjectsToUpdateConfig->Add(Settings, Settings->GetDefaultConfigFilename());
					}
				}

				ShowNotification(FText::Format(LOCTEXT("RemoveTagRedirect", "Deleted tag redirect {0}"), FText::FromName(TagToDelete)), 5.0f, false, bOnlyLog);

				WarnAboutRestart();

				if (bRefresh)
				{
					TSharedPtr<FGameplayTagNode> FoundNode = Manager.FindTagNode(TagToDelete);

					ensureMsgf(!FoundNode.IsValid() || FoundNode->GetCompleteTagName() == TagToDelete, TEXT("Failed to delete redirector %s!"), *TagToDelete.ToString());
				}

				return true;
			}
		}

		return false;
	}

	virtual bool AddNewGameplayTagToINI(const FString& NewTag, const FString& Comment, FName TagSourceName, bool bIsRestrictedTag, bool bAllowNonRestrictedChildren) override
	{
		UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

		if (NewTag.IsEmpty())
		{
			return false;
		}

		if (Manager.ShouldImportTagsFromINI() == false)
		{
			return false;
		}

		UGameplayTagsSettings*			Settings = GetMutableDefault<UGameplayTagsSettings>();
		UGameplayTagsDeveloperSettings* DevSettings = GetMutableDefault<UGameplayTagsDeveloperSettings>();

		FText ErrorText;
		FString FixedString;
		if (!Manager.IsValidGameplayTagString(NewTag, &ErrorText, &FixedString))
		{
			ShowNotification(FText::Format(LOCTEXT("AddTagFailure_BadString", "Failed to add gameplay tag {0}: {1}, try {2} instead!"), FText::FromString(NewTag), ErrorText, FText::FromString(FixedString)), 10.0f, true);
			return false;
		}

		FName NewTagName = FName(*NewTag);

		// Delete existing redirector
		DeleteTagRedirector(NewTagName);

		// Already in the list as an explicit tag, ignore. Note we want to add if it is in implicit tag. (E.g, someone added A.B.C then someone tries to add A.B)
		if (Manager.IsDictionaryTag(NewTagName))
		{
			ShowNotification(FText::Format(LOCTEXT("AddTagFailure_AlreadyExists", "Failed to add gameplay tag {0}, already exists!"), FText::FromString(NewTag)), 10.0f, true);

			return false;
		}

		if (bIsRestrictedTag)
		{
			// restricted tags can't be children of non-restricted tags
			FString AncestorTag = NewTag;
			bool bWasSplit = NewTag.Split(TEXT("."), &AncestorTag, nullptr, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
			while (bWasSplit)
			{
				if (Manager.IsDictionaryTag(FName(*AncestorTag)))
				{
					FString TagComment;
					FName Source;
					bool bIsExplicit;
					bool bIsRestricted;
					bool bAllowsNonRestrictedChildren;

					Manager.GetTagEditorData(*AncestorTag, TagComment, Source, bIsExplicit, bIsRestricted, bAllowsNonRestrictedChildren);
					if (bIsRestricted)
					{
						break;
					}
					ShowNotification(FText::Format(LOCTEXT("AddRestrictedTagFailure", "Failed to add restricted gameplay tag {0}, {1} is not a restricted tag"), FText::FromString(NewTag), FText::FromString(AncestorTag)), 10.0f, true);

					return false;
				}

				bWasSplit = AncestorTag.Split(TEXT("."), &AncestorTag, nullptr, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
			}
		}
		else
		{
			// non-restricted tags can only be children of restricted tags if the restricted tag allows it
			FString AncestorTag = NewTag;
			bool bWasSplit = NewTag.Split(TEXT("."), &AncestorTag, nullptr, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
			while (bWasSplit)
			{
				if (Manager.IsDictionaryTag(FName(*AncestorTag)))
				{
					FString TagComment;
					FName Source;
					bool bIsExplicit;
					bool bIsRestricted;
					bool bAllowsNonRestrictedChildren;

					Manager.GetTagEditorData(*AncestorTag, TagComment, Source, bIsExplicit, bIsRestricted, bAllowsNonRestrictedChildren);
					if (bIsRestricted)
					{
						if (bAllowsNonRestrictedChildren)
						{
							break;
						}

						ShowNotification(FText::Format(LOCTEXT("AddTagFailure_RestrictedTag", "Failed to add gameplay tag {0}, {1} is a restricted tag and does not allow non-restricted children"), FText::FromString(NewTag), FText::FromString(AncestorTag)), 10.0f, true);

						return false;
					}
				}

				bWasSplit = AncestorTag.Split(TEXT("."), &AncestorTag, nullptr, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
			}
		}

		if ((TagSourceName == NAME_None || TagSourceName == FGameplayTagSource::GetDefaultName()) && DevSettings && !DevSettings->DeveloperConfigName.IsEmpty())
		{
			// Try to use developer config file
			TagSourceName = FName(*FString::Printf(TEXT("%s.ini"), *DevSettings->DeveloperConfigName));
		}

		if (TagSourceName == NAME_None)
		{
			// If not set yet, set to default
			TagSourceName = FGameplayTagSource::GetDefaultName();
		}

		const FGameplayTagSource* TagSource = Manager.FindTagSource(TagSourceName);

		if (!TagSource)
		{
			// Create a new one
			TagSource = Manager.FindOrAddTagSource(TagSourceName, EGameplayTagSourceType::TagList);
		}

		bool bSuccess = false;
		if (TagSource)
		{
			UObject* TagListObj = nullptr;
			FString ConfigFileName;

			if (bIsRestrictedTag && TagSource->SourceRestrictedTagList)
			{
				URestrictedGameplayTagsList* RestrictedTagList = TagSource->SourceRestrictedTagList;
				TagListObj = RestrictedTagList;
				RestrictedTagList->RestrictedGameplayTagList.AddUnique(FRestrictedGameplayTagTableRow(FName(*NewTag), Comment, bAllowNonRestrictedChildren));
				RestrictedTagList->SortTags();
				ConfigFileName = RestrictedTagList->ConfigFileName;
				bSuccess = true;
			}
			else if (TagSource->SourceTagList)
			{
				UGameplayTagsList* TagList = TagSource->SourceTagList;
				TagListObj = TagList;
				TagList->GameplayTagList.AddUnique(FGameplayTagTableRow(FName(*NewTag), Comment));
				TagList->SortTags();
				ConfigFileName = TagList->ConfigFileName;
				bSuccess = true;
			}

			GameplayTagsUpdateSourceControl(ConfigFileName);

			// Check source control before and after writing, to make sure it gets created or checked out

			TagListObj->TryUpdateDefaultConfigFile(ConfigFileName);
			GameplayTagsUpdateSourceControl(ConfigFileName);
			GConfig->LoadFile(ConfigFileName);
		}
		
		if (!bSuccess)
		{
			ShowNotification(FText::Format(LOCTEXT("AddTagFailure", "Failed to add gameplay tag {0} to dictionary {1}!"), FText::FromString(NewTag), FText::FromName(TagSourceName)), 10.0f, true);

			return false;
		}

		{
			FString PerfMessage = FString::Printf(TEXT("ConstructGameplayTagTree GameplayTag tables after adding new tag"));
			SCOPE_LOG_TIME_IN_SECONDS(*PerfMessage, nullptr)

			Manager.EditorRefreshGameplayTagTree();
		}

		return true;
	}

	virtual bool DeleteTagFromINI(TSharedPtr<FGameplayTagNode> TagNodeToDelete) override
	{
		if (!TagNodeToDelete.IsValid())
		{
			return false;
		}

		TMap<UObject*, FString> ObjectsToUpdateConfig;
		const bool bOnlyLog = false;
		bool bReturnValue = DeleteTagFromINIInternal(TagNodeToDelete, bOnlyLog, ObjectsToUpdateConfig);
		if (ObjectsToUpdateConfig.Num() > 0)
		{
			UpdateTagSourcesAfterDelete(bOnlyLog, ObjectsToUpdateConfig);

			// This invalidates all local variables, need to return right away
			UGameplayTagsManager::Get().EditorRefreshGameplayTagTree();
		}
		return bReturnValue;
	}

	virtual void DeleteTagsFromINI(const TArray<TSharedPtr<FGameplayTagNode>>& TagNodesToDelete) override
	{
		TMap<UObject*, FString> ObjectsToUpdateConfig;
		const bool bOnlyLog = true;

		{
			FScopedSlowTask SlowTask(TagNodesToDelete.Num(), LOCTEXT("RemovingTags", "Removing Tags"));
			for (const TSharedPtr<FGameplayTagNode>& TagNodeToDelete : TagNodesToDelete)
			{
				if (ensure(!TagNodeToDelete->GetCompleteTagName().IsNone()))
				{
					SlowTask.EnterProgressFrame();
					if (TagNodeToDelete.IsValid())
					{
						DeleteTagFromINIInternal(TagNodeToDelete, bOnlyLog, ObjectsToUpdateConfig);
					}
					ensureMsgf(!TagNodeToDelete->GetCompleteTagName().IsNone(), TEXT("A 'None' tag here implies somone may have added a EditorRefreshGameplayTagTree() call in DeleteTagFromINI. Do not do this, the refresh must happen after the bulk operation is done."));
				}
			}
		}

		if (ObjectsToUpdateConfig.Num() > 0)
		{
			UpdateTagSourcesAfterDelete(bOnlyLog, ObjectsToUpdateConfig);

			UGameplayTagsManager& Manager = UGameplayTagsManager::Get();
			Manager.EditorRefreshGameplayTagTree();
		}
	}

	bool DeleteTagFromINIInternal(const TSharedPtr<FGameplayTagNode>& TagNodeToDelete, bool bOnlyLog, TMap<UObject*, FString>& OutObjectsToUpdateConfig)
	{
		FName TagName = TagNodeToDelete->GetCompleteTagName();

		UGameplayTagsManager& Manager = UGameplayTagsManager::Get();
		UGameplayTagsSettings* Settings = GetMutableDefault<UGameplayTagsSettings>();

		FString Comment;
		TArray<FName> TagSourceNames;
		bool bTagIsExplicit;
		bool bTagIsRestricted;
		bool bTagAllowsNonRestrictedChildren;

		if (DeleteTagRedirector(TagName, bOnlyLog, false, &OutObjectsToUpdateConfig))
		{
			return true;
		}
		
		if (!Manager.GetTagEditorData(TagName, Comment, TagSourceNames, bTagIsExplicit, bTagIsRestricted, bTagAllowsNonRestrictedChildren))
		{
			ShowNotification(FText::Format(LOCTEXT("RemoveTagFailureNoTag", "Cannot delete tag {0}, does not exist!"), FText::FromName(TagName)), 10.0f, true, bOnlyLog);
			return false;
		}

		ensure(bTagIsRestricted == TagNodeToDelete->IsRestrictedGameplayTag());

		// Check if the tag is implicitly defined
		if (!bTagIsExplicit || TagSourceNames.Num() == 0)
		{
			ShowNotification(FText::Format(LOCTEXT("RemoveTagFailureNoSource", "Cannot delete tag {0} as it is implicit, remove children manually"), FText::FromName(TagName)), 10.0f, true, bOnlyLog);
			return false;
		}

		FGameplayTag ActualTag = Manager.RequestGameplayTag(TagName);
		FGameplayTagContainer ChildTags = Manager.RequestGameplayTagChildrenInDictionary(ActualTag);

		TArray<FName> TagsThatWillBeDeleted;

		TagsThatWillBeDeleted.Add(TagName);

		FGameplayTag ParentTag = ActualTag.RequestDirectParent();
		while (ParentTag.IsValid() && !Manager.FindTagNode(ParentTag)->IsExplicitTag())
		{
			// See if there are more children than the one we are about to delete
			FGameplayTagContainer ParentChildTags = Manager.RequestGameplayTagChildrenInDictionary(ParentTag);

			ensure(ParentChildTags.HasTagExact(ActualTag));
			if (ParentChildTags.Num() == 1)
			{
				// This is the only tag, add to deleted list
				TagsThatWillBeDeleted.Add(ParentTag.GetTagName());
				ParentTag = ParentTag.RequestDirectParent();
			}
			else
			{
				break;
			}
		}

		for (FName TagNameToDelete : TagsThatWillBeDeleted)
		{
			// Verify references
			FAssetIdentifier TagId = FAssetIdentifier(FGameplayTag::StaticStruct(), TagNameToDelete);
			TArray<FAssetIdentifier> Referencers;

			FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
			AssetRegistryModule.Get().GetReferencers(TagId, Referencers, UE::AssetRegistry::EDependencyCategory::SearchableName);

			if (Referencers.Num() > 0)
			{
				ShowNotification(FText::Format(LOCTEXT("RemoveTagFailureBadSource_Referenced", "Cannot delete tag {0}, still referenced by {1} and possibly others"), FText::FromName(TagNameToDelete), FText::FromString(Referencers[0].ToString())), 10.0f, true, bOnlyLog);

				return false;
			}
		}

		bool bRemovedAny = false;
		for (const FName& TagSourceName : TagSourceNames)
		{
			const FGameplayTagSource* TagSource = Manager.FindTagSource(TagSourceName);

			if (bTagIsRestricted && !TagSource->SourceRestrictedTagList)
			{
				ShowNotification(FText::Format(LOCTEXT("RemoveTagFailureBadSource", "Cannot delete tag {0} from source {1}, remove manually"), FText::FromName(TagName), FText::FromName(TagSourceName)), 10.0f, true, bOnlyLog);
				continue;
			}

			if (!bTagIsRestricted && !TagSource->SourceTagList)
			{
				ShowNotification(FText::Format(LOCTEXT("RemoveTagFailureBadSource", "Cannot delete tag {0} from source {1}, remove manually"), FText::FromName(TagName), FText::FromName(TagSourceName)), 10.0f, true, bOnlyLog);
				continue;
			}

			// Passed, delete and save
			const FString& ConfigFileName = bTagIsRestricted ? TagSource->SourceRestrictedTagList->ConfigFileName : TagSource->SourceTagList->ConfigFileName;

			int32 NumRemoved = 0;
			if (bTagIsRestricted)
			{
				NumRemoved = TagSource->SourceRestrictedTagList->RestrictedGameplayTagList.RemoveAll([&TagName](const FRestrictedGameplayTagTableRow& Row) { return Row.Tag == TagName; });
				if (NumRemoved > 0)
				{
					OutObjectsToUpdateConfig.Add(TagSource->SourceRestrictedTagList, ConfigFileName);
				}
			}
			else
			{
				NumRemoved = TagSource->SourceTagList->GameplayTagList.RemoveAll([&TagName](const FGameplayTagTableRow& Row) { return Row.Tag == TagName; });
				if (NumRemoved > 0)
				{
					OutObjectsToUpdateConfig.Add(TagSource->SourceTagList, ConfigFileName);
				}
			}

			if (NumRemoved > 0)
			{
				// See if we still live due to child tags
				if (ChildTags.Num() > 0)
				{
					ShowNotification(FText::Format(LOCTEXT("RemoveTagChildrenExist", "Deleted explicit tag {0}, still exists implicitly due to children"), FText::FromName(TagName)), 5.0f, false, bOnlyLog);
				}
				else
				{
					ShowNotification(FText::Format(LOCTEXT("RemoveTag", "Deleted tag {0}"), FText::FromName(TagName)), 5.0f, false, bOnlyLog);
				}

				bRemovedAny = true;
			}
		}

		if (!bRemovedAny)
		{
			ShowNotification(FText::Format(LOCTEXT("RemoveTagFailureNoTag", "Cannot delete tag {0}, does not exist!"), FText::FromName(TagName)), 10.0f, true, bOnlyLog);
		}
		
		return bRemovedAny;
	}

	void UpdateTagSourcesAfterDelete(bool bOnlyLog, const TMap<UObject*, FString>& ObjectsToUpdateConfig)
	{
		TSet<FString> ConfigFileNames;
		for (auto ObjectToUpdateIt = ObjectsToUpdateConfig.CreateConstIterator(); ObjectToUpdateIt; ++ObjectToUpdateIt)
		{
			ConfigFileNames.Add(ObjectToUpdateIt.Value());
		}

		FScopedSlowTask SlowTask(ObjectsToUpdateConfig.Num() + ConfigFileNames.Num(), LOCTEXT("UpdateTagSourcesAfterDelete", "Updating Tag Sources"));

		GameplayTagsUpdateSourceControl(ConfigFileNames.Array(), bOnlyLog);

		for (auto ObjectToUpdateIt = ObjectsToUpdateConfig.CreateConstIterator(); ObjectToUpdateIt; ++ObjectToUpdateIt)
		{
			SlowTask.EnterProgressFrame();
			check(ObjectToUpdateIt.Key());
			ObjectToUpdateIt.Key()->TryUpdateDefaultConfigFile(ObjectToUpdateIt.Value());
		}

		for (const FString& ConfigFileName : ConfigFileNames)
		{
			SlowTask.EnterProgressFrame();
			GConfig->LoadFile(ConfigFileName);
		}
	}

	virtual bool UpdateTagInINI(const FString& TagToUpdate, const FString& Comment, bool bIsRestrictedTag, bool bAllowNonRestrictedChildren) override
	{
		FName TagName = FName(*TagToUpdate);

		UGameplayTagsManager& Manager = UGameplayTagsManager::Get();
		UGameplayTagsSettings* Settings = GetMutableDefault<UGameplayTagsSettings>();

		FString OldComment;
		FName TagSourceName;
		bool bTagIsExplicit;
		bool bTagWasRestricted;
		bool bTagDidAllowNonRestrictedChildren;

		bool bSuccess = false;

		if (Manager.GetTagEditorData(TagName, OldComment, TagSourceName, bTagIsExplicit, bTagWasRestricted, bTagDidAllowNonRestrictedChildren))
		{
			if (const FGameplayTagSource* TagSource = Manager.FindTagSource(TagSourceName))
			{
				// if we're disallowing non-restricted children make sure we don't already have some
				if (bTagDidAllowNonRestrictedChildren && !bAllowNonRestrictedChildren)
				{
					FGameplayTag ActualTag = Manager.RequestGameplayTag(TagName);
					FGameplayTagContainer ChildTags = Manager.RequestGameplayTagDirectDescendantsInDictionary(ActualTag, EGameplayTagSelectionType::NonRestrictedOnly);
					if (!ChildTags.IsEmpty())
					{
						ShowNotification(LOCTEXT("ToggleAllowNonRestrictedChildrenFailure", "Cannot prevent non-restricted children since some already exist! Delete them first."), 10.0f, true);
						return false;
					}
				}

				UObject* TagListObj = nullptr;
				FString ConfigFileName;

				if (bIsRestrictedTag && TagSource->SourceRestrictedTagList)
				{
					URestrictedGameplayTagsList* RestrictedTagList = TagSource->SourceRestrictedTagList;
					TagListObj = RestrictedTagList;
					ConfigFileName = RestrictedTagList->ConfigFileName;

					for (int32 i = 0; i < RestrictedTagList->RestrictedGameplayTagList.Num(); i++)
					{
						if (RestrictedTagList->RestrictedGameplayTagList[i].Tag == TagName)
						{
							RestrictedTagList->RestrictedGameplayTagList[i].bAllowNonRestrictedChildren = bAllowNonRestrictedChildren;
							bSuccess = true;
							break;
						}
					}
				}

				if (bSuccess)
				{
					// Check source control before and after writing, to make sure it gets created or checked out
					GameplayTagsUpdateSourceControl(ConfigFileName);
					TagListObj->TryUpdateDefaultConfigFile(ConfigFileName);
					GameplayTagsUpdateSourceControl(ConfigFileName);

					GConfig->LoadFile(ConfigFileName);
				}

			}
		}

		return bSuccess;
	}

	virtual bool RenameTagInINI(const FString& TagToRename, const FString& TagToRenameTo, bool bRenameChildren) override
	{
		FName OldTagName = FName(*TagToRename);
		FName NewTagName = FName(*TagToRenameTo);

		UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

		if (bRenameChildren)
		{
			FGameplayTag ActualTag = Manager.RequestGameplayTag(OldTagName);
			const FGameplayTagContainer Children = Manager.RequestGameplayTagDirectDescendantsInDictionary(ActualTag, EGameplayTagSelectionType::All);

			for (const FGameplayTag& ChildTag : Children.GetGameplayTagArray())
			{
				FString NewTag = ChildTag.ToString().Replace(*(TagToRename + '.'), *(TagToRenameTo + '.'), ESearchCase::CaseSensitive);
				RenameTagInINI(ChildTag.ToString(), NewTag, true);
			}
		}

		FString OldComment, NewComment;
		FName OldTagSourceName, NewTagSourceName;
		bool bTagIsExplicit;
		bool bTagIsRestricted;
		bool bTagAllowsNonRestrictedChildren;

		// Delete existing redirector
		DeleteTagRedirector(NewTagName);
		DeleteTagRedirector(OldTagName);

		const FGameplayTagSource* OldTagSource = nullptr;
		if (Manager.GetTagEditorData(OldTagName, OldComment, OldTagSourceName, bTagIsExplicit, bTagIsRestricted, bTagAllowsNonRestrictedChildren))
		{
			// Add new tag if needed
			if (!Manager.GetTagEditorData(NewTagName, NewComment, NewTagSourceName, bTagIsExplicit, bTagIsRestricted, bTagAllowsNonRestrictedChildren))
			{
				if (!AddNewGameplayTagToINI(TagToRenameTo, OldComment, OldTagSourceName, bTagIsRestricted, bTagAllowsNonRestrictedChildren))
				{
					// Failed to add new tag, so fail
					return false;
				}
			}

			// Delete old tag if possible, still make redirector if this fails
			OldTagSource = Manager.FindTagSource(OldTagSourceName);

			if (OldTagSource && OldTagSource->SourceTagList)
			{
				UGameplayTagsList* TagList = OldTagSource->SourceTagList;

				for (int32 i = 0; i < TagList->GameplayTagList.Num(); i++)
				{
					if (TagList->GameplayTagList[i].Tag == OldTagName)
					{
						TagList->GameplayTagList.RemoveAt(i);

						TagList->TryUpdateDefaultConfigFile(TagList->ConfigFileName);
						GameplayTagsUpdateSourceControl(TagList->ConfigFileName);
						GConfig->LoadFile(TagList->ConfigFileName);

						break;
					}
				}
			}
			else
			{
				ShowNotification(FText::Format(LOCTEXT("RenameFailure", "Tag {0} redirector was created but original tag was not destroyed as it has children"), FText::FromString(TagToRename)), 10.0f, true);
			}
		}

		// Add redirector no matter what
		FGameplayTagRedirect Redirect;
		Redirect.OldTagName = OldTagName;
		Redirect.NewTagName = NewTagName;

		UGameplayTagsList* ListToUpdate = (OldTagSource && OldTagSource->SourceTagList) ? OldTagSource->SourceTagList.Get() : GetMutableDefault<UGameplayTagsSettings>();
		check(ListToUpdate);

		ListToUpdate->GameplayTagRedirects.AddUnique(Redirect);

		GameplayTagsUpdateSourceControl(ListToUpdate->ConfigFileName);
		ListToUpdate->TryUpdateDefaultConfigFile(ListToUpdate->ConfigFileName);
		GConfig->LoadFile(ListToUpdate->ConfigFileName);

		ShowNotification(FText::Format(LOCTEXT("AddTagRedirect", "Renamed tag {0} to {1}"), FText::FromString(TagToRename), FText::FromString(TagToRenameTo)), 3.0f);

		Manager.EditorRefreshGameplayTagTree();

		WarnAboutRestart();

		return true;
	}

	virtual bool MoveTagsBetweenINI(const TArray<FString>& TagsToMove, const FName& TargetTagSource, TArray<FString>& OutTagsMoved, TArray<FString>& OutFailedToMoveTags) override
	{
		UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

		// Find and check out the destination .ini file
		FGameplayTagSource* NewTagSource = Manager.FindTagSource(TargetTagSource);
		if (NewTagSource == nullptr)
		{
			ShowNotification(FText::Format(LOCTEXT("MoveTagsFailure_UnknownTarget", "Failed to move tags as target {0} could not be found"), FText::FromName(TargetTagSource)), 10.0f, true);
			return false;
		}

		if (NewTagSource->SourceType != EGameplayTagSourceType::DefaultTagList && NewTagSource->SourceType != EGameplayTagSourceType::TagList)
		{
			ShowNotification(FText::Format(LOCTEXT("MoveTagsFailure_UnsupportedTarget", "Invalid target source `{0}`! Tags can only be moved to DefaultTagList and TagList target sources."), FText::FromName(TargetTagSource)), 10.0f, true);
			return false;
		}

		ensure(NewTagSource->SourceTagList);
		
		// Tracking which lists are modified for bulk operations (checkout, config file update/reload).
		TSet<UGameplayTagsList*> ModifiedTagsList;

		// For each gameplay tag, remove it from the current GameplayTagList and add it to the destination
		for (const FString& TagToMove : TagsToMove)
		{
			FString Comment;
			TArray<FName> TagSourceNames;
			bool bIsTagExplicit, bIsRestrictedTag, bAllowNonRestrictedChildren;
			Manager.GetTagEditorData(FName(TagToMove), Comment, TagSourceNames, bIsTagExplicit, bIsRestrictedTag, bAllowNonRestrictedChildren);

			if (bIsRestrictedTag)
			{
				ShowNotification(FText::Format(LOCTEXT("MoveTagsFailure_RestrictedTag", "Restriced Tag {0} cannot be moved"), FText::FromString(TagToMove)), 10.0f, true);
				OutFailedToMoveTags.Add(TagToMove);
				continue;
			}

			if (TagSourceNames.IsEmpty())
			{
				ShowNotification(FText::Format(LOCTEXT("MoveTagsFailure_UnknownSource", "Tag {0} sources could not be found"), FText::FromString(TagToMove)), 10.0f, true);
				OutFailedToMoveTags.Add(TagToMove);
				continue;
			}
			else if (TagSourceNames.Num() > 1)
			{
				UE_LOGF(LogGameplayTags, Display, "%d tag sources found for tag %ls. Moving first found ini source only! (DefaultTagList or TagList)", TagSourceNames.Num(), *TagToMove);
			}

			// Move the first found .ini tag list only
			FGameplayTagSource* OldTagSource = nullptr;
			for (FName TagSourceName : TagSourceNames)
			{
				FGameplayTagSource* TagSource = Manager.FindTagSource(TagSourceName);
				if (TagSource != nullptr && (TagSource->SourceType == EGameplayTagSourceType::DefaultTagList || TagSource->SourceType == EGameplayTagSourceType::TagList))
				{
					OldTagSource = TagSource;
					break;
				}
			}

			if (OldTagSource == nullptr)
			{
				ShowNotification(FText::Format(LOCTEXT("MoveTagsFailure_InvalidSource", "Invalid source for `{0}`! Tags can only be moved from DefaultTagList and TagList sources."), FText::FromString(TagToMove)), 10.0f, true);
				OutFailedToMoveTags.Add(TagToMove);
				continue;
			}

			ensure(OldTagSource->SourceTagList);

			bool bFound = false;
			FName TagToMoveName(TagToMove);

			// Remove from the old tag source
			TArray<FGameplayTagTableRow>& GameplayTagList = OldTagSource->SourceTagList->GameplayTagList;
			for (int32 Index = 0; Index < GameplayTagList.Num(); ++Index)
			{
				if (GameplayTagList[Index].Tag == TagToMoveName)
				{
					GameplayTagList.RemoveAt(Index);
					ModifiedTagsList.Add(OldTagSource->SourceTagList);
					bFound = true;
					break;
				}
			}

			if (bFound)
			{
				// Add to the new tag source
				NewTagSource->SourceTagList->GameplayTagList.AddUnique(FGameplayTagTableRow(FName(TagToMove), Comment));
				ModifiedTagsList.Add(NewTagSource->SourceTagList);

				OutTagsMoved.Add(TagToMove);
			}
			else
			{
				ShowNotification(FText::Format(LOCTEXT("MoveTagsFailure_TagNotFound", "Tag {0} could not be found in the source tag list {0}"), FText::FromString(TagToMove), FText::FromString(OldTagSource->SourceTagList->ConfigFileName)), 10.0f, true);
				OutFailedToMoveTags.Add(TagToMove);
			}
		}

		// Update all modified tags list
		for (UGameplayTagsList* TagsList : ModifiedTagsList)
		{
			GameplayTagsUpdateSourceControl(TagsList->ConfigFileName);

			TagsList->SortTags();
			TagsList->TryUpdateDefaultConfigFile(TagsList->ConfigFileName);

			GConfig->LoadFile(TagsList->ConfigFileName);
		};

		// Refresh editor
		Manager.EditorRefreshGameplayTagTree();

		return true;
	}

	virtual bool AddTransientEditorGameplayTag(const FString& NewTransientTag) override
	{
		UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

		if (NewTransientTag.IsEmpty())
		{
			return false;
		}

		Manager.TransientEditorTags.Add(*NewTransientTag);

		{
			FString PerfMessage = FString::Printf(TEXT("ConstructGameplayTagTree GameplayTag tables after adding new transient tag"));
			SCOPE_LOG_TIME_IN_SECONDS(*PerfMessage, nullptr)

			Manager.EditorRefreshGameplayTagTree();
		}

		return true;
	}

	virtual bool AddNewGameplayTagSource(const FString& NewTagSource, const FString& RootDirToUse = FString()) override
	{
		UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

		if (NewTagSource.IsEmpty())
		{
			return false;
		}

		// Tag lists should always end with .ini
		FName TagSourceName;
		if (NewTagSource.EndsWith(TEXT(".ini")))
		{
			TagSourceName = FName(*NewTagSource);
		}
		else
		{
			TagSourceName = FName(*(NewTagSource + TEXT(".ini")));
		}

		Manager.FindOrAddTagSource(TagSourceName, EGameplayTagSourceType::TagList, RootDirToUse);

		ShowNotification(FText::Format(LOCTEXT("AddTagSource", "Added {0} as a source for saving new tags"), FText::FromName(TagSourceName)), 3.0f);

		IGameplayTagsModule::OnTagSettingsChanged.Broadcast();

		return true;
	}

	TSharedRef<SWidget> MakeGameplayTagContainerWidget(FOnSetGameplayTagContainer OnSetTag, TSharedPtr<FGameplayTagContainer> GameplayTagContainer, const FString& FilterString) override
	{
		if (!GameplayTagContainer.IsValid())
		{
			return SNullWidget::NullWidget;
		}

		TArray<FGameplayTagContainer> EditableContainers;
		EditableContainers.Emplace(*GameplayTagContainer);

		SGameplayTagPicker::FOnTagChanged OnChanged = SGameplayTagPicker::FOnTagChanged::CreateLambda([OnSetTag, GameplayTagContainer](const TArray<FGameplayTagContainer>& TagContainers)
		{
			if (TagContainers.Num() > 0)
			{
				*GameplayTagContainer.Get() = TagContainers[0];
				OnSetTag.Execute(*GameplayTagContainer.Get());
			}
		});

		return SNew(SGameplayTagPicker)
			.TagContainers(EditableContainers)
			.Filter(FilterString)
			.ReadOnly(false)
			.MultiSelect(true)
			.OnTagChanged(OnChanged);
	}

	TSharedRef<SWidget> MakeGameplayTagWidget(FOnSetGameplayTag OnSetTag, TSharedPtr<FGameplayTag> GameplayTag, const FString& FilterString) override
	{
		if (!GameplayTag.IsValid())
		{
			return SNullWidget::NullWidget;
		}

		TArray<FGameplayTagContainer> EditableContainers;
		EditableContainers.Emplace(*GameplayTag);

		SGameplayTagPicker::FOnTagChanged OnChanged = SGameplayTagPicker::FOnTagChanged::CreateLambda([OnSetTag, GameplayTag](const TArray<FGameplayTagContainer>& TagContainers)
		{
			if (TagContainers.Num() > 0)
			{
				*GameplayTag.Get() = TagContainers[0].First();
				OnSetTag.Execute(*GameplayTag.Get());
			}
		});

		return SNew(SGameplayTagPicker)
			.TagContainers(EditableContainers)
			.Filter(FilterString)
			.ReadOnly(false)
			.MultiSelect(false)
			.OnTagChanged(OnChanged);
	}

	void GetUnusedGameplayTags(TArray<TSharedPtr<FGameplayTagNode>>& OutUnusedTags) override
	{
		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
		UGameplayTagsManager& Manager = UGameplayTagsManager::Get();
		int32 NumUsedExplicitTags = 0;
		TSet<FString> AllConfigValues;
		TMultiMap<FName, FName> RerverseRedirectorMap;

		// Populate all config values from all config files so that we can check if any config contains a reference to a tag later
		{
			TArray<FString> AllConfigFilenames;
			GConfig->GetConfigFilenames(AllConfigFilenames);

			for (const FString& ConfigFilename : AllConfigFilenames)
			{
				if (FConfigFile* ConfigFile = GConfig->FindConfigFile(ConfigFilename))
				{
					for (const TPair<FString, FConfigSection>& FileIt : AsConst(*ConfigFile))
					{
						const FString& SectionName = FileIt.Key;

						// Do not include sections that define the tags, as we don't wan't their definition showing up as a reference
						if (SectionName != TEXT("/Script/GameplayTags.GameplayTagsSettings") && SectionName != TEXT("/Script/GameplayTags.GameplayTagsList"))
						{
							for (const TPair<FName, FConfigValue>& SectionIt : FileIt.Value)
							{
								const FString& ConfigValue = SectionIt.Value.GetValue();

								// Cut down on values by skipping pure numbers
								if (!ConfigValue.IsNumeric())
								{
									AllConfigValues.Add(ConfigValue);
								}
							}
						}
					}
				}
			}
		}

		// Build a reverse map of tag redirectors so we can find the old names of a given tag to see if any of them are still referenced
		UGameplayTagsSettings* Settings = GetMutableDefault<UGameplayTagsSettings>();
		for (const FGameplayTagRedirect& Redirect : Settings->GameplayTagRedirects)
		{
			RerverseRedirectorMap.Add(Redirect.NewTagName, Redirect.OldTagName);
		}

		FScopedSlowTask SlowTask((float)Manager.GetNumGameplayTagNodes(), LOCTEXT("PopulatingUnusedTags", "Populating Unused Tags"));

		// Function to determine if a single node is referenced by content
		auto IsNodeUsed = [&AssetRegistry, &AllConfigValues, &RerverseRedirectorMap](const TSharedPtr<FGameplayTagNode>& Node) -> bool
		{
			// Look for references to the input tag or any of its old names if there were redirectors
			FName InitialTagName = Node->GetCompleteTagName();
			TArray<FName> TagsToCheck = { InitialTagName };
			RerverseRedirectorMap.MultiFind(InitialTagName, TagsToCheck);

			for (const FName& TagName : TagsToCheck)
			{
				// Look for asset references
				FAssetIdentifier TagId = FAssetIdentifier(FGameplayTag::StaticStruct(), TagName);
				TArray<FAssetIdentifier> Referencers;
				AssetRegistry.GetReferencers(TagId, Referencers, UE::AssetRegistry::EDependencyCategory::SearchableName);
				if (Referencers.Num() != 0)
				{
					return true;
				}

				// Look for config references
				FString TagString = TagName.ToString();
				for (const FString& ConfigValue : AllConfigValues)
				{
					if (ConfigValue.Contains(TagString))
					{
						return true;
					}
				}
			}

			return false;
		};

		// Recursive function to traverse the Gameplay Tag Node tree and find all unused tags
		TFunction<void(const TSharedPtr<FGameplayTagNode>&)> RecursiveProcessTagNode;
		RecursiveProcessTagNode = [&RecursiveProcessTagNode, &NumUsedExplicitTags, &Manager, &SlowTask, &IsNodeUsed, &OutUnusedTags](const TSharedPtr<FGameplayTagNode>& Node)
		{
			check(Node.IsValid());

			SlowTask.EnterProgressFrame();

			if (Node->IsExplicitTag())
			{
				bool bSourcesDetectable = false;
				for (const FName& SourceName : Node->GetAllSourceNames())
				{
					const FGameplayTagSource* TagSource = Manager.FindTagSource(SourceName);
					if (TagSource &&
						(TagSource->SourceType == EGameplayTagSourceType::DefaultTagList || TagSource->SourceType == EGameplayTagSourceType::TagList || TagSource->SourceType == EGameplayTagSourceType::RestrictedTagList))
					{
						bSourcesDetectable = true;
					}
					else
					{
						bSourcesDetectable = false;
						break;
					}
				}

				if (bSourcesDetectable && !IsNodeUsed(Node))
				{
					OutUnusedTags.Add(Node);
				}
				else
				{
					NumUsedExplicitTags++;
				}
			}

			// Iterate children recursively
			const int32 NumUsedExplicitTagsBeforeChildren = NumUsedExplicitTags;
			const int32 NumUnusedExplicitTagsBeforeChildren = OutUnusedTags.Num();
			const TArray<TSharedPtr<FGameplayTagNode>>& ChildNodes = Node->GetChildTagNodes();
			for (const TSharedPtr<FGameplayTagNode>& ChildNode : ChildNodes)
			{
				RecursiveProcessTagNode(ChildNode);
			}

			// Implicit tags need at least one explicit child in order to exist.
			// If an implicit tag is referenced, treat the last explicit child as being referenced too
			const bool bAtLeastOneUsedExplicitChild = NumUsedExplicitTags > NumUsedExplicitTagsBeforeChildren;
			const bool bAtLeastOneUnusedExplicitChild = OutUnusedTags.Num() > NumUnusedExplicitTagsBeforeChildren;
			if (!Node->IsExplicitTag() && !bAtLeastOneUsedExplicitChild && bAtLeastOneUnusedExplicitChild)
			{
				// This is an implicit tag that has only unused explicit children. If this tag is referenced, then remove the last unused child from the list
				if (IsNodeUsed(Node))
				{
					// This implicit tag is referenced. Remove the last unused child as this tag is using that child in order to exist.
					OutUnusedTags.RemoveAt(OutUnusedTags.Num() - 1, 1, EAllowShrinking::No);
					NumUsedExplicitTags++;
				}
			}
		};

		// Go through all root nodes to process entire tree
		TArray<TSharedPtr<FGameplayTagNode>> AllRoots;
		Manager.GetFilteredGameplayRootTags(TEXT(""), AllRoots);
		for (const TSharedPtr<FGameplayTagNode>& Root : AllRoots)
		{
			RecursiveProcessTagNode(Root);
		}
	}

	static bool WriteCustomReport(FString FileName, TArray<FString>& FileLines)
	{
		// Has a report been generated
		bool ReportGenerated = false;

		// Ensure we have a log to write
		if (FileLines.Num())
		{
			// Create the file name		
			FString FileLocation = FPaths::ConvertRelativePathToFull(FPaths::ProjectSavedDir() + TEXT("Reports/"));
			FString FullPath = FString::Printf(TEXT("%s%s"), *FileLocation, *FileName);

			// save file
			FArchive* LogFile = IFileManager::Get().CreateFileWriter(*FullPath);

			if (LogFile != NULL)
			{
				for (int32 Index = 0; Index < FileLines.Num(); ++Index)
				{
					FString LogEntry = FString::Printf(TEXT("%s"), *FileLines[Index]) + LINE_TERMINATOR;
					LogFile->Serialize(TCHAR_TO_ANSI(*LogEntry), LogEntry.Len());
				}

				LogFile->Close();
				delete LogFile;

				// A report has been generated
				ReportGenerated = true;
			}
		}

		return ReportGenerated;
	}

	static void DumpTagList()
	{
		UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

		TArray<FString> ReportLines;
		TArray<FString> ReportReferencers;
		TArray<FString> ReportSources;

		ReportLines.Add(TEXT("Tag,Explicit,HasNativeSource,HasConfigSource,Reference Count,Sources Count,Comment"));
		ReportReferencers.Add(TEXT("Asset,Tag"));
		ReportSources.Add(TEXT("Source,Tag"));

		FGameplayTagContainer AllTags;
		Manager.RequestAllGameplayTags(AllTags, true);

		TArray<FGameplayTag> ExplicitList;
		AllTags.GetGameplayTagArray(ExplicitList);

		ExplicitList.Sort();

		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		
		for (const FGameplayTag& Tag : ExplicitList)
		{
			TArray<FAssetIdentifier> Referencers;
			FAssetIdentifier TagId = FAssetIdentifier(FGameplayTag::StaticStruct(), Tag.GetTagName());
			AssetRegistryModule.Get().GetReferencers(TagId, Referencers, UE::AssetRegistry::EDependencyCategory::SearchableName);

			FString Comment;
			TArray<FName> TagSources;
			bool bExplicit, bRestricted, bAllowNonRestrictedChildren;

			Manager.GetTagEditorData(Tag.GetTagName(), Comment, TagSources, bExplicit, bRestricted, bAllowNonRestrictedChildren);

			bool bHasNative = TagSources.Contains(FGameplayTagSource::GetNativeName());
			bool bHasConfigIni = TagSources.Contains(FGameplayTagSource::GetDefaultName());

			FString TagName = Tag.ToString();

			ReportLines.Add(FString::Printf(TEXT("%s,%s,%s,%s,%d,%d,\"%s\""),
				*TagName,
				bExplicit ? TEXT("true") : TEXT("false"),
				bHasNative ? TEXT("true") : TEXT("false"),
				bHasConfigIni ? TEXT("true") : TEXT("false"),
				Referencers.Num(),
				TagSources.Num(),
				*Comment));

			for (const FAssetIdentifier& Referencer : Referencers)
			{
				ReportReferencers.Add(FString::Printf(TEXT("%s,%s"), *Referencer.ToString(), *TagName));
			}

			for (const FName& TagSource : TagSources)
			{
				ReportSources.Add(FString::Printf(TEXT("%s,%s"), *TagSource.ToString(), *TagName));
			}
		}

		WriteCustomReport(TEXT("TagList.csv"), ReportLines);
		WriteCustomReport(TEXT("TagReferencesList.csv"), ReportReferencers);
		WriteCustomReport(TEXT("TagSourcesList.csv"), ReportSources);
	}

	FDelegateHandle AssetImportHandle;
	FDelegateHandle SettingsChangedHandle;

	FName GameplayTagPackageName;
	FName GameplayTagStructName;
};

static FAutoConsoleCommand CVarDumpTagList(
	TEXT("GameplayTags.DumpTagList"),
	TEXT("Writes out a csvs with all tags to Reports/TagList.csv, ")
	TEXT("Reports/TagReferencesList.csv and Reports/TagSourcesList.csv"),
	FConsoleCommandDelegate::CreateStatic(FGameplayTagsEditorModule::DumpTagList),
	ECVF_Cheat);

IMPLEMENT_MODULE(FGameplayTagsEditorModule, GameplayTagsEditor)

#undef LOCTEXT_NAMESPACE
