// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraphUtilities.h"
#include "SGraphNode_MultiCompareGameplayTag.h"
#include "GameplayTagsK2Node_MultiCompareBase.h"

class FGameplayTagsGraphPanelNodeFactory : public FGraphPanelNodeFactory
{
	virtual TSharedPtr<class SGraphNode> CreateNode(class UEdGraphNode* InNode) const override
	{
		if (UGameplayTagsK2Node_MultiCompareBase* CompareNode = Cast<UGameplayTagsK2Node_MultiCompareBase>(InNode))
		{
			return SNew(SGraphNode_MultiCompareGameplayTag, CompareNode);
		}

		return NULL;
	}
};
