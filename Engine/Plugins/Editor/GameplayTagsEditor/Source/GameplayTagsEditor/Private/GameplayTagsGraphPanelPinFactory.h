// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraphUtilities.h"
#include "GameplayTagContainer.h"
#include "EdGraphSchema_K2.h"
#include "SGameplayTagGraphPin.h"
#include "SGameplayTagContainerGraphPin.h"
#include "SGameplayTagQueryGraphPin.h"

class FGameplayTagsGraphPanelPinFactory: public FGraphPanelPinFactory
{
	virtual TSharedPtr<class SGraphPin> CreatePin(class UEdGraphPin* InPin) const override
	{
		if (InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct)
		{
			if (UScriptStruct* PinStructType = Cast<UScriptStruct>(InPin->PinType.PinSubCategoryObject.Get()))
			{
				if (PinStructType->IsChildOf(FGameplayTag::StaticStruct()))
				{
					return SNew(SGameplayTagGraphPin, InPin);
				}
				else if (PinStructType->IsChildOf(FGameplayTagContainer::StaticStruct()))
				{
					return SNew(SGameplayTagContainerGraphPin, InPin);
				}
				else if (PinStructType->IsChildOf(FGameplayTagQuery::StaticStruct()))
				{
					return SNew(SGameplayTagQueryGraphPin, InPin);
				}
			}
		}
		else if (InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_String && InPin->PinType.PinSubCategory == TEXT("LiteralGameplayTagContainer"))
		{
			return SNew(SGameplayTagContainerGraphPin, InPin);
		}

		return nullptr;
	}
};
