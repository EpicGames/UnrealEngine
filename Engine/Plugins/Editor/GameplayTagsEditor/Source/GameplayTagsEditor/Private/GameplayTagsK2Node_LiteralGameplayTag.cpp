// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTagsK2Node_LiteralGameplayTag.h"
#include "EdGraphSchema_K2.h"

#include "GameplayTagContainer.h"
#include "BlueprintGameplayTagLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayTagsK2Node_LiteralGameplayTag)

#define LOCTEXT_NAMESPACE "GameplayTagsK2Node_LiteralGameplayTag"

UGameplayTagsK2Node_LiteralGameplayTag::UGameplayTagsK2Node_LiteralGameplayTag(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

void UGameplayTagsK2Node_LiteralGameplayTag::AllocateDefaultPins()
{
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_String, TEXT("LiteralGameplayTagContainer"), TEXT("TagIn"));
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Struct, FGameplayTagContainer::StaticStruct(), UEdGraphSchema_K2::PN_ReturnValue);
}

FLinearColor UGameplayTagsK2Node_LiteralGameplayTag::GetNodeTitleColor() const
{
	return FLinearColor(1.0f, 0.51f, 0.0f);
}

FText UGameplayTagsK2Node_LiteralGameplayTag::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return NSLOCTEXT("K2Node", "LiteralGameplayTag", "Make Literal GameplayTagContainer");
}

bool UGameplayTagsK2Node_LiteralGameplayTag::CanCreateUnderSpecifiedSchema( const UEdGraphSchema* Schema ) const
{
	return Schema->IsA(UEdGraphSchema_K2::StaticClass());
}

void UGameplayTagsK2Node_LiteralGameplayTag::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	ensureMsgf(0, TEXT("GameplayTagsK2Node_LiteralGameplayTag is deprecated and should never make it to compile time"));
}

void UGameplayTagsK2Node_LiteralGameplayTag::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	
}

FText UGameplayTagsK2Node_LiteralGameplayTag::GetMenuCategory() const
{
	return LOCTEXT("ActionMenuCategory", "Gameplay Tags");
}

FEdGraphNodeDeprecationResponse UGameplayTagsK2Node_LiteralGameplayTag::GetDeprecationResponse(EEdGraphNodeDeprecationType DeprecationType) const
{
	FEdGraphNodeDeprecationResponse Response = Super::GetDeprecationResponse(DeprecationType);
	Response.MessageText = LOCTEXT("NodeDeprecated_Warning", "@@ is deprecated, replace with Make Literal GameplayTagContainer function call");

	return Response;
}

void UGameplayTagsK2Node_LiteralGameplayTag::ConvertDeprecatedNode(UEdGraph* Graph, bool bOnlySafeChanges)
{
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	TMap<FName, FName> OldPinToNewPinMap;

	UFunction* MakeFunction = UBlueprintGameplayTagLibrary::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UBlueprintGameplayTagLibrary, MakeLiteralGameplayTagContainer));
	OldPinToNewPinMap.Add(TEXT("TagIn"), TEXT("Value"));

	ensure(Schema->ConvertDeprecatedNodeToFunctionCall(this, MakeFunction, OldPinToNewPinMap, Graph) != nullptr);
}

#undef LOCTEXT_NAMESPACE

