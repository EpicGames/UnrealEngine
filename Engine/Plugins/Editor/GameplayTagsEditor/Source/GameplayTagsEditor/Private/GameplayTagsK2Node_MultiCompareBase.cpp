// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTagsK2Node_MultiCompareBase.h"
#include "EdGraph/EdGraph.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayTagsK2Node_MultiCompareBase)

UGameplayTagsK2Node_MultiCompareBase::UGameplayTagsK2Node_MultiCompareBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	NumberOfPins = 1;
	OrphanedPinSaveMode = ESaveOrphanPinMode::SaveNone;
}

void UGameplayTagsK2Node_MultiCompareBase::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	// If the number of pins is changed mark the node as dirty and reconstruct
	const FName PropertyName = (PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None);
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UGameplayTagsK2Node_MultiCompareBase, NumberOfPins))
	{
		if (NumberOfPins < 0)
		{
			NumberOfPins = 1;
		}

		ReconstructNode();
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
	GetGraph()->NotifyNodeChanged(this);
}

FText UGameplayTagsK2Node_MultiCompareBase::GetTooltipText() const
{
	return NSLOCTEXT("K2Node", "MultiCompareTagContainer_ToolTip", "Sets the an output for each input value");
}

FText UGameplayTagsK2Node_MultiCompareBase::GetMenuCategory() const
{
	return NSLOCTEXT("K2Node", "MultiCompareTagContainer_ActionMenuCategory", "Gameplay Tags|Tag Container");
}

FString UGameplayTagsK2Node_MultiCompareBase::GetUniquePinName()
{
	FString NewPinName;
	int32 Index = 0;
	while (true)
	{
		NewPinName = FString::Printf(TEXT("Case_%d"), Index++);
		bool bFound = false;
		for (int32 PinIdx = 0; PinIdx < PinNames.Num(); PinIdx++)
		{
			FString PinName = PinNames[PinIdx].ToString();
			if (PinName == NewPinName)
			{
				bFound = true;
				break;
			}			
		}
		if (!bFound)
		{
			break;
		}
	}
	return NewPinName;
}

void UGameplayTagsK2Node_MultiCompareBase::AddPin()
{
	NumberOfPins++;
}

void UGameplayTagsK2Node_MultiCompareBase::RemovePin()
{
	if (NumberOfPins > 1)
	{
		NumberOfPins--;
	}	
}

