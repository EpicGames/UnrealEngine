// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTagsK2Node_MultiCompareGameplayTagContainer.h"
#include "K2Node_CallFunction.h"
#include "BlueprintNodeSpawner.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "GameplayTagContainer.h"
#include "BlueprintGameplayTagLibrary.h"
#include "KismetCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayTagsK2Node_MultiCompareGameplayTagContainer)

UGameplayTagsK2Node_MultiCompareGameplayTagContainer::UGameplayTagsK2Node_MultiCompareGameplayTagContainer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UGameplayTagsK2Node_MultiCompareGameplayTagContainer::AllocateDefaultPins()
{
	PinNames.Reset();
	for (int32 Index = 0; Index < NumberOfPins; ++Index)
	{
		AddPinToSwitchNode();
	}

	UEdGraphNode::FCreatePinParams PinParams;
	PinParams.bIsReference = true;
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Struct, FGameplayTagContainer::StaticStruct(), TEXT("Gameplay Tag Container"));
}

void UGameplayTagsK2Node_MultiCompareGameplayTagContainer::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	// Get The input and output pins to our node
	UEdGraphPin* InPinSwitch = FindPin(TEXT("Gameplay Tag Container"));

	// For Each Pin Compare against the Tag container
	for (int32 Index = 0; Index < NumberOfPins; ++Index)
	{
		const FString InPinName = TEXT("TagCase_") + FString::FormatAsNumber(Index);
		const FString OutPinName = TEXT("Case_") + FString::FormatAsNumber(Index) + TEXT(" True");
		UEdGraphPin* InPinCase = FindPin(InPinName);
		UEdGraphPin* OutPinCase = FindPin(OutPinName);

		// Create call function node for the Compare function HasAllMatchingGameplayTags
		UK2Node_CallFunction* PinCallFunction = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
		const UFunction* Function = UBlueprintGameplayTagLibrary::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UBlueprintGameplayTagLibrary, HasAllTags));
		PinCallFunction->SetFromFunction(Function);
		PinCallFunction->AllocateDefaultPins();

		UEdGraphPin *TagContainerPin = PinCallFunction->FindPinChecked(TEXT("TagContainer"));
		CompilerContext.CopyPinLinksToIntermediate(*InPinSwitch, *TagContainerPin);

		UEdGraphPin *OtherTagContainerPin = PinCallFunction->FindPinChecked(TEXT("OtherContainer"));
		CompilerContext.MovePinLinksToIntermediate(*InPinCase, *OtherTagContainerPin);
		
		UEdGraphPin *OutPin = PinCallFunction->FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue);

		if (OutPinCase && OutPin)
		{
			OutPin->PinType = OutPinCase->PinType; // Copy type so it uses the right actor subclass
			CompilerContext.MovePinLinksToIntermediate(*OutPinCase, *OutPin);
		}
	}

	// Break any links to the expanded node
	BreakAllNodeLinks();
}

FText UGameplayTagsK2Node_MultiCompareGameplayTagContainer::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return NSLOCTEXT("K2Node", "MultiCompare_TagContainer", "Compare Tag Container to Other Tag Containers");
}

void UGameplayTagsK2Node_MultiCompareGameplayTagContainer::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

void UGameplayTagsK2Node_MultiCompareGameplayTagContainer::AddPinToSwitchNode()
{
	const FString PinName = GetUniquePinName();
	const FName InPin = *(TEXT("Tag") + PinName);
	const FName OutPin = *(PinName + TEXT(" True"));
	PinNames.Add(FName(*PinName));

	UEdGraphNode::FCreatePinParams InPinParams;
	InPinParams.bIsReference = true;

	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Struct, FGameplayTagContainer::StaticStruct(), InPin, InPinParams);
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Boolean, OutPin);
}

