// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTagsK2Node_SwitchGameplayTag.h"
#include "EdGraph/EdGraph.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "BlueprintNodeSpawner.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintGameplayTagLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayTagsK2Node_SwitchGameplayTag)

UGameplayTagsK2Node_SwitchGameplayTag::UGameplayTagsK2Node_SwitchGameplayTag(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	FunctionName = TEXT("NotEqual_TagTag");
	FunctionClass = UBlueprintGameplayTagLibrary::StaticClass();
	OrphanedPinSaveMode = ESaveOrphanPinMode::SaveNone;
}

void UGameplayTagsK2Node_SwitchGameplayTag::CreateFunctionPin()
{
	// Set properties on the function pin
	UEdGraphPin* FunctionPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Object, FunctionClass, FunctionName);
	FunctionPin->bDefaultValueIsReadOnly = true;
	FunctionPin->bNotConnectable = true;
	FunctionPin->bHidden = true;

	UFunction* Function = FindUField<UFunction>(FunctionClass, FunctionName);
	const bool bIsStaticFunc = Function->HasAllFunctionFlags(FUNC_Static);
	if (bIsStaticFunc)
	{
		// Wire up the self to the CDO of the class if it's not us
		if (UBlueprint* BP = GetBlueprint())
		{
			UClass* FunctionOwnerClass = Function->GetOuterUClass();
			if (!BP->SkeletonGeneratedClass->IsChildOf(FunctionOwnerClass))
			{
				FunctionPin->DefaultObject = FunctionOwnerClass->GetDefaultObject();
			}
		}
	}
}

void UGameplayTagsK2Node_SwitchGameplayTag::PostLoad()
{
	Super::PostLoad();
	if (UEdGraphPin* FunctionPin = FindPin(FunctionName))
	{
		FunctionPin->DefaultObject = FunctionClass->GetDefaultObject();
	}
}

void UGameplayTagsK2Node_SwitchGameplayTag::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	bool bIsDirty = false;
	const FName PropertyName = (PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None);
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UGameplayTagsK2Node_SwitchGameplayTag, PinTags))
	{
		bIsDirty = true;
	}

	if (bIsDirty)
	{
		ReconstructNode();
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
	GetGraph()->NotifyNodeChanged(this);
}

FText UGameplayTagsK2Node_SwitchGameplayTag::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return NSLOCTEXT("K2Node", "Switch_Tag", "Switch on Gameplay Tag");
}

FText UGameplayTagsK2Node_SwitchGameplayTag::GetTooltipText() const
{
	return NSLOCTEXT("K2Node", "SwitchTag_ToolTip", "Selects an output that matches the input value");
}

void UGameplayTagsK2Node_SwitchGameplayTag::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

void UGameplayTagsK2Node_SwitchGameplayTag::CreateSelectionPin()
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	UEdGraphPin* Pin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Struct, FGameplayTag::StaticStruct(), TEXT("Selection"));
	K2Schema->SetPinAutogeneratedDefaultValueBasedOnType(Pin);
}

FEdGraphPinType UGameplayTagsK2Node_SwitchGameplayTag::GetPinType() const
{
	FEdGraphPinType PinType;
	PinType.PinCategory = UEdGraphSchema_K2::PC_Struct;
	PinType.PinSubCategoryObject = FGameplayTag::StaticStruct();
	return PinType;
}

FEdGraphPinType UGameplayTagsK2Node_SwitchGameplayTag::GetInnerCaseType() const
{
	// This type should match the second argument of UBlueprintGameplayTagLibrary::NotEqual_TagTag !
	FEdGraphPinType PinType;
	PinType.PinCategory = UEdGraphSchema_K2::PC_String;
	return PinType;
}

FName UGameplayTagsK2Node_SwitchGameplayTag::GetPinNameGivenIndex(int32 Index) const
{
	check(Index);
	return PinNames[Index];
}

void UGameplayTagsK2Node_SwitchGameplayTag::CreateCasePins()
{
	while (PinTags.Num() > PinNames.Num())
	{
		const FName PinName = GetUniquePinName();
		PinNames.Add(PinName);
	}

	if (PinNames.Num() > PinTags.Num())
	{
		PinNames.SetNum(PinTags.Num());
	}

	for (int32 Index = 0; Index < PinTags.Num(); ++Index)
  	{
		if (PinTags[Index].IsValid())
		{
			PinNames[Index] = PinTags[Index].GetTagName();
		}			
		else
		{
			PinNames[Index] = GetUniquePinName();
		}

		CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, PinNames[Index]);
  	}
}

FName UGameplayTagsK2Node_SwitchGameplayTag::GetUniquePinName()
{
	FName NewPinName;
	int32 Index = 0;
	while (true)
	{
		NewPinName = *FString::Printf(TEXT("Case_%d"), Index++);
		if (!FindPin(NewPinName))
		{
			break;
		}
	}
	return NewPinName;
}

void UGameplayTagsK2Node_SwitchGameplayTag::AddPinToSwitchNode()
{
	FName PinName = GetUniquePinName();
	PinNames.Add(PinName);

	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, PinName);
	if (PinTags.Num() < PinNames.Num())
	{ 
		PinTags.Add(FGameplayTag());
	}
}

void UGameplayTagsK2Node_SwitchGameplayTag::RemovePin(UEdGraphPin* TargetPin)
{
	checkSlow(TargetPin);

	FName PinName = TargetPin->PinName;
	// Clean-up pin name array
	int32 Index = PinNames.IndexOfByKey(PinName);
	if (Index >= 0)
	{
		if (Index < PinTags.Num())
		{
			PinTags.RemoveAt(Index);
		}
		PinNames.RemoveAt(Index);
	}
}

