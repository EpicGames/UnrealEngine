// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTagsSettingsCustomization.h"
#include "GameplayTagsSettings.h"
#include "GameplayTagsModule.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "SGameplayTagPicker.h"
#include "SAddNewGameplayTagSourceWidget.h"
#include "SCleanupUnusedGameplayTagsWidget.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "FGameplayTagsSettingsCustomization"

TSharedRef<IDetailCustomization> FGameplayTagsSettingsCustomization::MakeInstance()
{
	return MakeShareable( new FGameplayTagsSettingsCustomization() );
}

FGameplayTagsSettingsCustomization::FGameplayTagsSettingsCustomization()
{
}

FGameplayTagsSettingsCustomization::~FGameplayTagsSettingsCustomization()
{
	IGameplayTagsModule::OnTagSettingsChanged.RemoveAll(this);
}

void FGameplayTagsSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	IDetailCategoryBuilder& GameplayTagsCategory = DetailLayout.EditCategory("GameplayTags");
	{
		TArray<TSharedRef<IPropertyHandle>> GameplayTagsProperties;
		GameplayTagsCategory.GetDefaultProperties(GameplayTagsProperties, true, true);

		TSharedPtr<IPropertyHandle> TagListProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UGameplayTagsList, GameplayTagList), UGameplayTagsList::StaticClass());
		TagListProperty->MarkHiddenByCustomization();

		TSharedPtr<IPropertyHandle> NewTagSourceProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UGameplayTagsSettings, NewTagSource));
		NewTagSourceProperty->MarkHiddenByCustomization();

		TSharedPtr<IPropertyHandle> CleanupUnusedTagsProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UGameplayTagsSettings, CleanupUnusedTags));
		CleanupUnusedTagsProperty->MarkHiddenByCustomization();

		for (TSharedPtr<IPropertyHandle> Property : GameplayTagsProperties)
		{
			if (Property->GetProperty() == TagListProperty->GetProperty())
			{
				// Button to open tag manager
				GameplayTagsCategory.AddCustomRow(TagListProperty->GetPropertyDisplayName(), /*bForAdvanced*/false)
				.NameContent()
				[
					TagListProperty->CreatePropertyNameWidget()
				]
				.ValueContent()
				[
					SNew(SButton)
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					.OnClicked_Lambda([this]()
					{
						FGameplayTagManagerWindowArgs Args;
						Args.bRestrictedTags = false;
						UE::GameplayTags::Editor::OpenGameplayTagManager(Args);
						return FReply::Handled();
					})
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(FMargin(0,0,4,0))
						[
							SNew( SImage )
							.Image(FAppStyle::GetBrush("Icons.Settings"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(STextBlock)
							.Text(LOCTEXT("ManageGameplayTags", "Manage Gameplay Tags..."))
						]
					]
				];
			}
			else if (Property->GetProperty() ==  NewTagSourceProperty->GetProperty())
			{
				// Button to open add source dialog
				GameplayTagsCategory.AddCustomRow(NewTagSourceProperty->GetPropertyDisplayName(), /*bForAdvanced*/false)
				.NameContent()
				[
					NewTagSourceProperty->CreatePropertyNameWidget()
				]
				.ValueContent()
				[
					SNew(SButton)
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					.OnClicked_Lambda([this]()
					{
						const TSharedRef<SWindow> Window = SNew(SWindow)
							.Title(LOCTEXT("AddNewGameplayTagSourceTitle", "Add new Gameplay Tag Source"))
							.SizingRule(ESizingRule::Autosized)
							.SupportsMaximize(false)
							.SupportsMinimize(false)
							.Content()
							[
								SNew(SBox)
								.MinDesiredWidth(320.0f)
								[
									SNew(SAddNewGameplayTagSourceWidget)
								]
							];

						GEditor->EditorAddModalWindow(Window);

						return FReply::Handled();
					})
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(FMargin(0,0,4,0))
						[
							SNew( SImage )
							.Image(FAppStyle::GetBrush("Icons.Plus"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(STextBlock)
							.Text(LOCTEXT("AddNewGameplayTagSource", "Add new Gameplay Tag source..."))
						]
					]
				];
			}
			else if (Property->GetProperty() == CleanupUnusedTagsProperty->GetProperty())
			{
				// Button to open add source dialog
				GameplayTagsCategory.AddCustomRow(CleanupUnusedTagsProperty->GetPropertyDisplayName(), /*bForAdvanced*/false)
				.NameContent()
				[
					CleanupUnusedTagsProperty->CreatePropertyNameWidget()
				]
				.ValueContent()
				[
					SNew(SButton)
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					.OnClicked_Lambda([this]()
					{
						const TSharedRef<SWindow> Window = SNew(SWindow)
							.Title(LOCTEXT("CleanupUnusedTagsTitle", "Cleanup Unused Tags"))
							.SizingRule(ESizingRule::UserSized)
							.ClientSize(FVector2D(700, 700))
							.SupportsMinimize(false)
							.Content()
							[
								SNew(SBox)
								.MinDesiredWidth(100.f)
								.MinDesiredHeight(100.f)
								[
									SNew(SCleanupUnusedGameplayTagsWidget)
								]
							];

						TSharedPtr<SWindow> RootWindow = FGlobalTabmanager::Get()->GetRootWindow();
						if (RootWindow.IsValid())
						{
							FSlateApplication::Get().AddWindowAsNativeChild(Window, RootWindow.ToSharedRef());
						}
						else
						{
							FSlateApplication::Get().AddWindow(Window);
						}

						return FReply::Handled();
					})
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(FMargin(0,0,4,0))
						[
							SNew( SImage )
							.Image(FAppStyle::GetBrush("Icons.Delete"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(STextBlock)
							.Text(LOCTEXT("CleanupUnusedTags", "Cleanup Unused Tags..."))
						]
					]
				];
			}
			else
			{
				GameplayTagsCategory.AddProperty(Property);
			}
		}
	}

	IDetailCategoryBuilder& AdvancedGameplayTagsCategory = DetailLayout.EditCategory("Advanced Gameplay Tags");
	{
		TArray<TSharedRef<IPropertyHandle>> GameplayTagsProperties;
		AdvancedGameplayTagsCategory.GetDefaultProperties(GameplayTagsProperties, true, true);

		TSharedPtr<IPropertyHandle> RestrictedTagListProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UGameplayTagsSettings, RestrictedTagList));
		RestrictedTagListProperty->MarkHiddenByCustomization();

		for (TSharedPtr<IPropertyHandle> Property : GameplayTagsProperties)
		{
			if (Property->GetProperty() == RestrictedTagListProperty->GetProperty())
			{
				// Button to open restricted tag manager
				AdvancedGameplayTagsCategory.AddCustomRow(RestrictedTagListProperty->GetPropertyDisplayName(), true)
				.NameContent()
				[
					RestrictedTagListProperty->CreatePropertyNameWidget()
				]
				.ValueContent()
				[
					SNew(SButton)
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					.OnClicked_Lambda([this]()
					{
						FGameplayTagManagerWindowArgs Args;
						Args.bRestrictedTags = true;
						UE::GameplayTags::Editor::OpenGameplayTagManager(Args);
						return FReply::Handled();
					})
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(FMargin(0,0,4,0))
						[
							SNew( SImage )
							.Image(FAppStyle::GetBrush("Icons.Settings"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(STextBlock)
							.Text(LOCTEXT("ManageRestrictedGameplayTags", "Manage Restricted Gameplay Tags..."))
						]
					]
				];
			}
			else
			{
				AdvancedGameplayTagsCategory.AddProperty(Property);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
