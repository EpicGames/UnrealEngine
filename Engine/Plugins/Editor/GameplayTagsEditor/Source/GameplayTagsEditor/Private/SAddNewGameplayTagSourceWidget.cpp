// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAddNewGameplayTagSourceWidget.h"
#include "DetailLayoutBuilder.h"
#include "Framework/Views/TableViewMetadata.h"
#include "GameplayTagsEditorModule.h"
#include "GameplayTagsManager.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "Misc/Paths.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SGridPanel.h"

#define LOCTEXT_NAMESPACE "AddNewGameplayTagSourceWidget"

void SAddNewGameplayTagSourceWidget::Construct(const FArguments& InArgs)
{
	FText HintText = LOCTEXT("NewSourceNameHint", "SourceName.ini");
	DefaultNewName = InArgs._NewSourceName;
	if (DefaultNewName.IsEmpty() == false)
	{
		HintText = FText::FromString(DefaultNewName);
	}

	bShouldGetKeyboardFocus = false;

	OnGameplayTagSourceAdded = InArgs._OnGameplayTagSourceAdded;
	PopulateTagRoots();

	ChildSlot
	[
		SNew(SBox)
		.Padding(InArgs._Padding)
		[
			SNew(SGridPanel)
			.FillColumn(1, 1.0)

			// Tag Name
			+ SGridPanel::Slot(0, 0)
			.Padding(2)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			[
				SNew(SBox)
				.MinDesiredWidth(50.0f)
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont")))
					.Text(LOCTEXT("NewSourceName", "Name:"))
				]
			]
			+ SGridPanel::Slot(1, 0)
			.Padding(2)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			[
				SNew(SBox)
				.MinDesiredWidth(250.0f)
				[
					SAssignNew(SourceNameTextBox, SEditableTextBox)
					.HintText(HintText)
				]
			]

			// Tag source root
			+ SGridPanel::Slot(0, 1)
			.Padding(2)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ConfigPathLabel", "Config Path:"))
				.Font(FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont")))
				.ToolTipText(LOCTEXT("RootPathTooltip", "Set the base config path for added source, this includes paths from plugins and other places that call AddTagIniSearchPath"))
			]
			+ SGridPanel::Slot(1, 1)
			.Padding(2)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			[
				SAssignNew(TagRootsComboBox, SSearchableComboBox)
				.OptionsSource(&TagRoots)
				.OnGenerateWidget(this, &SAddNewGameplayTagSourceWidget::OnGenerateTagRootsComboBox)
				.Content()
				[
					SNew(STextBlock)
					.Text(this, &SAddNewGameplayTagSourceWidget::CreateTagRootsComboBoxContent)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			]
			
			// Add Source Button
			+ SGridPanel::Slot(0, 2)
			.ColumnSpan(2)
			.Padding(FMargin(0, 16))
			.HAlign(HAlign_Right)
			[
				SNew(SButton)
				.Text(LOCTEXT("AddNew", "Add New Source"))
				.OnClicked(this, &SAddNewGameplayTagSourceWidget::OnAddNewSourceButtonPressed)
			]
		]
	];

	Reset();
}

void SAddNewGameplayTagSourceWidget::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	if (bShouldGetKeyboardFocus)
	{
		bShouldGetKeyboardFocus = false;
		FSlateApplication::Get().SetKeyboardFocus(SourceNameTextBox.ToSharedRef(), EFocusCause::SetDirectly);
	}
}

void SAddNewGameplayTagSourceWidget::Reset()
{
	SetSourceName();
}

void SAddNewGameplayTagSourceWidget::SetSourceName(const FText& InName)
{
	SourceNameTextBox->SetText(InName.IsEmpty() ? FText::FromString(DefaultNewName) : InName);
}

void SAddNewGameplayTagSourceWidget::PopulateTagRoots()
{
	UGameplayTagsManager& Manager = UGameplayTagsManager::Get();
	TagRoots.Empty();

	FName DefaultSource = FGameplayTagSource::GetDefaultName();

	TArray<FString> TagRootStrings;
	Manager.GetTagSourceSearchPaths(TagRootStrings);

	for (const FString& TagRoot : TagRootStrings)
	{
		TagRoots.Add(MakeShareable(new FString(TagRoot)));
	}
}

FText SAddNewGameplayTagSourceWidget::GetFriendlyPath(TSharedPtr<FString> InItem) const
{
	if (InItem.IsValid())
	{
		FString FilePath = *InItem.Get();

		if (FPaths::IsUnderDirectory(FilePath, FPaths::ProjectDir()))
		{
			FPaths::MakePathRelativeTo(FilePath, *FPaths::ProjectDir());
		}
		return FText::FromString(FilePath);
	}
	return FText();
}

TSharedRef<SWidget> SAddNewGameplayTagSourceWidget::OnGenerateTagRootsComboBox(TSharedPtr<FString> InItem)
{
	return SNew(STextBlock)
		.Text(GetFriendlyPath(InItem));
}

FText SAddNewGameplayTagSourceWidget::CreateTagRootsComboBoxContent() const
{
	const bool bHasSelectedItem = TagRootsComboBox.IsValid() && TagRootsComboBox->GetSelectedItem().IsValid();

	if (bHasSelectedItem)
	{
		return GetFriendlyPath(TagRootsComboBox->GetSelectedItem());
	}
	else
	{
		return LOCTEXT("NewTagRootNotSelected", "Default");
	}
}

FReply SAddNewGameplayTagSourceWidget::OnAddNewSourceButtonPressed()
{
	UGameplayTagsManager& Manager = UGameplayTagsManager::Get();
		
	if (!SourceNameTextBox->GetText().EqualTo(FText::FromString(DefaultNewName)))
	{
		FString TagRoot;
		if (TagRootsComboBox->GetSelectedItem().Get())
		{
			TagRoot = *TagRootsComboBox->GetSelectedItem().Get();
		}

		IGameplayTagsEditorModule::Get().AddNewGameplayTagSource(SourceNameTextBox->GetText().ToString(), TagRoot);
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
