// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class SEditableTextBox;
class SSearchableComboBox;

/** Widget allowing the user to create new gameplay tags */
class SAddNewGameplayTagSourceWidget : public SCompoundWidget
{
public:

	DECLARE_DELEGATE_OneParam( FOnGameplayTagSourceAdded, const FString& /*SourceName*/);

	SLATE_BEGIN_ARGS(SAddNewGameplayTagSourceWidget)
		: _NewSourceName(TEXT(""))
		, _Padding(FMargin(15))
		{}
		SLATE_EVENT(FOnGameplayTagSourceAdded, OnGameplayTagSourceAdded )	// Callback for when a new source is added	
		SLATE_ARGUMENT( FString, NewSourceName ) // String that will initially populate the New Source Name field
		SLATE_ARGUMENT(FMargin, Padding)
	SLATE_END_ARGS();

	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

	void Construct( const FArguments& InArgs);

	/** Resets all input fields */
	void Reset();

private:

	/** Sets the name of the source. Uses the default if the name is not specified */
	void SetSourceName(const FText& InName = FText());

	/** Creates a list of all root directories for tag sources */
	void PopulateTagRoots();

	/** Populates the widget's combo box with all potential places where a tag source can be stored */
	TSharedRef<SWidget> OnGenerateTagRootsComboBox(TSharedPtr<FString> InItem);

	/** Gets friendly version of a root path */
	FText GetFriendlyPath(TSharedPtr<FString> InItem) const;

	/** Creates the text displayed by the combo box when an option is selected */
	FText CreateTagRootsComboBoxContent() const;

	/** Callback for when the Add New Tag button is pressed */
	FReply OnAddNewSourceButtonPressed();

	/** The name of the next gameplay tag source to create */
	TSharedPtr<SEditableTextBox> SourceNameTextBox;

	/** Callback for when a new gameplay tag has been added to the INI files */
	FOnGameplayTagSourceAdded OnGameplayTagSourceAdded;

	/** All potential INI files where a tag source can be stored */
	TArray<TSharedPtr<FString> > TagRoots;

	/** The directory where the next tag source will be created */
	TSharedPtr<SSearchableComboBox> TagRootsComboBox;

	/** Tracks if this widget should get keyboard focus */
	bool bShouldGetKeyboardFocus;

	FString DefaultNewName;
};
