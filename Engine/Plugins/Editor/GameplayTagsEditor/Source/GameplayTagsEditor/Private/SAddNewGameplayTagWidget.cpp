// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAddNewGameplayTagWidget.h"
#include "DetailLayoutBuilder.h"
#include "GameplayTagsEditorModule.h"
#include "GameplayTagsManager.h"
#include "GameplayTagsModule.h"
#include "Misc/Paths.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "Misc/MessageDialog.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"

#define LOCTEXT_NAMESPACE "AddNewGameplayTagWidget"

SAddNewGameplayTagWidget::~SAddNewGameplayTagWidget()
{
	if (!GExitPurge)
	{
		IGameplayTagsModule::OnTagSettingsChanged.RemoveAll(this);
	}
}

void SAddNewGameplayTagWidget::Construct(const FArguments& InArgs)
{
	FText HintText = LOCTEXT("NewTagNameHint", "X.Y.Z");
	DefaultNewName = InArgs._NewTagName;
	if (DefaultNewName.IsEmpty() == false)
	{
		HintText = FText::FromString(DefaultNewName);
	}

	bAddingNewTag = false;
	bShouldGetKeyboardFocus = false;
	bRestrictedTags = InArgs._RestrictedTags;

	OnGameplayTagAdded = InArgs._OnGameplayTagAdded;
	IsValidTag = InArgs._IsValidTag;
	PopulateTagSources();

	IGameplayTagsModule::OnTagSettingsChanged.AddRaw(this, &SAddNewGameplayTagWidget::PopulateTagSources);

	ChildSlot
	[
		SNew(SBox)
		.Padding(InArgs._Padding)
		[
			SNew(SGridPanel)
			.FillColumn(1, 1.0)
			
			// Tag Name
			+ SGridPanel::Slot(0, 0)
			.Padding(2)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			[
				SNew(STextBlock)
				.Font(FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont")))
				.Text(LOCTEXT("NewTagName", "Name:"))
			]
			+ SGridPanel::Slot(1, 0)
			.Padding(2)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			[
				SAssignNew(TagNameTextBox, SEditableTextBox)
				.HintText(HintText)
				.Font(FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont")))
				.OnTextCommitted(this, &SAddNewGameplayTagWidget::OnCommitNewTagName)
			]
			
			// Tag Comment
			+ SGridPanel::Slot(0, 1)
			.Padding(2)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			[
				SNew(STextBlock)
				.Font(FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont")))
				.Text(LOCTEXT("TagComment", "Comment:"))
			]
			+ SGridPanel::Slot(1, 1)
			.Padding(2)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			[
				SAssignNew(TagCommentTextBox, SEditableTextBox)
				.HintText(LOCTEXT("TagCommentHint", "Comment"))
				.Font(FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont")))
				.OnTextCommitted(this, &SAddNewGameplayTagWidget::OnCommitNewTagName)
			]

			// Tag Location
			+ SGridPanel::Slot(0, 2)
			.Padding(2)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("CreateTagSource", "Source:"))
				.Font(FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont")))
			]
			+ SGridPanel::Slot(1, 2)
			.Padding(2)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SAssignNew(TagSourcesComboBox, SSearchableComboBox)
					.OptionsSource(&TagSources)
					.OnGenerateWidget(this, &SAddNewGameplayTagWidget::OnGenerateTagSourcesComboBox)
					.ToolTipText(this, &SAddNewGameplayTagWidget::CreateTagSourcesComboBoxToolTip)
					.Content()
					[
						SNew(STextBlock)
						.Text(this, &SAddNewGameplayTagWidget::CreateTagSourcesComboBoxContent)
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.0f, 0, 0, 0)
				.VAlign(VAlign_Center)
				[
					SNew( SButton )
					.ButtonStyle( FAppStyle::Get(), "NoBorder" )
					.Visibility(this, &SAddNewGameplayTagWidget::OnGetTagSourceFavoritesVisibility)
					.OnClicked(this, &SAddNewGameplayTagWidget::OnToggleTagSourceFavoriteClicked)
					.ToolTipText(LOCTEXT("ToggleFavoriteTooltip", "Toggle whether or not this tag source is your favorite source (new tags will go into your favorite source by default)"))
					.ContentPadding(0)
					[
						SNew(SImage)
						.Image(this, &SAddNewGameplayTagWidget::OnGetTagSourceFavoriteImage)
					]
				]
			]

			// Add Tag Button
			+ SGridPanel::Slot(0, 3)
			.ColumnSpan(2)
			.Padding(InArgs._AddButtonPadding)
			.HAlign(HAlign_Right)
			[
				SNew(SButton)
				.Text(LOCTEXT("AddNew", "Add New Tag"))
				.OnClicked(this, &SAddNewGameplayTagWidget::OnAddNewTagButtonPressed)
			]
		]
	];

	Reset(EResetType::ResetAll);
}

EVisibility SAddNewGameplayTagWidget::OnGetTagSourceFavoritesVisibility() const
{
	return (TagSources.Num() > 1) ? EVisibility::Visible : EVisibility::Collapsed;
}

FReply SAddNewGameplayTagWidget::OnToggleTagSourceFavoriteClicked()
{
	const FName ActiveTagSource = GetSelectedTagSource();
	const bool bWasFavorite = FGameplayTagSource::GetFavoriteName() == ActiveTagSource;

	FGameplayTagSource::SetFavoriteName(bWasFavorite ? NAME_None : ActiveTagSource);

	return FReply::Handled();
}

const FSlateBrush* SAddNewGameplayTagWidget::OnGetTagSourceFavoriteImage() const
{
	const FName ActiveTagSource = GetSelectedTagSource();
	const bool bIsFavoriteTagSource = FGameplayTagSource::GetFavoriteName() == ActiveTagSource;

	return FAppStyle::GetBrush(bIsFavoriteTagSource ? TEXT("Icons.Star") : TEXT("PropertyWindow.Favorites_Disabled"));
}

void SAddNewGameplayTagWidget::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	if (bShouldGetKeyboardFocus)
	{
		bShouldGetKeyboardFocus = false;
		FSlateApplication::Get().SetKeyboardFocus(TagNameTextBox.ToSharedRef(), EFocusCause::SetDirectly);
		FSlateApplication::Get().SetUserFocus(0, TagNameTextBox.ToSharedRef());
	}
}

void SAddNewGameplayTagWidget::PopulateTagSources()
{
	const UGameplayTagsManager& Manager = UGameplayTagsManager::Get();
	TagSources.Empty();

	TArray<const FGameplayTagSource*> Sources;

	if (bRestrictedTags)
	{
		Manager.GetRestrictedTagSources(Sources);

		// Add the placeholder source if no other sources exist
		if (Sources.Num() == 0)
		{
			TagSources.Add(MakeShareable(new FString()));
		}

		for (const FGameplayTagSource* Source : Sources)
		{
			if (Source != nullptr && !Source->SourceName.IsNone())
			{
				TagSources.Add(MakeShareable(new FString(Source->SourceName.ToString())));
			}
		}
	}
	else
	{
		const FName DefaultSource = FGameplayTagSource::GetDefaultName();

		// Always ensure that the default source is first
		TagSources.Add( MakeShareable( new FString( DefaultSource.ToString() ) ) );

		Manager.FindTagSourcesWithType(EGameplayTagSourceType::TagList, Sources);

		Algo::SortBy(Sources, &FGameplayTagSource::SourceName, FNameLexicalLess());

		for (const FGameplayTagSource* Source : Sources)
		{
			if (Source != nullptr && Source->SourceName != DefaultSource)
			{
				TagSources.Add(MakeShareable(new FString(Source->SourceName.ToString())));
			}
		}

		//Set selection to the latest added source
		if (TagSourcesComboBox != nullptr)
		{
			TagSourcesComboBox->SetSelectedItem(TagSources.Last());
		}
	}
}

void SAddNewGameplayTagWidget::Reset(EResetType ResetType)
{
	SetTagName();
	if (ResetType != EResetType::DoNotResetSource)
	{
		SelectTagSource(FGameplayTagSource::GetFavoriteName());
	}
	TagCommentTextBox->SetText(FText());
}

void SAddNewGameplayTagWidget::SetTagName(const FText& InName)
{
	TagNameTextBox->SetText(InName.IsEmpty() ? FText::FromString(DefaultNewName) : InName);
}

void SAddNewGameplayTagWidget::SelectTagSource(const FName& InSource)
{
	// Attempt to find the location in our sources, otherwise just use the first one
	int32 SourceIndex = INDEX_NONE;

	if (!InSource.IsNone())
	{
		for (int32 Index = 0; Index < TagSources.Num(); ++Index)
		{
			TSharedPtr<FString> Source = TagSources[Index];

			if (Source.IsValid() && *Source.Get() == InSource.ToString())
			{
				SourceIndex = Index;
				break;
			}
		}
	}

	if (SourceIndex != INDEX_NONE && TagSourcesComboBox.IsValid())
	{
		TagSourcesComboBox->SetSelectedItem(TagSources[SourceIndex]);
	}
}

FName SAddNewGameplayTagWidget::GetSelectedTagSource() const
{
	const bool bHasSelectedItem = TagSourcesComboBox.IsValid() && TagSourcesComboBox->GetSelectedItem().IsValid();

	if (bHasSelectedItem)
	{
		// Convert to FName which the rest of the API expects
		return FName(**TagSourcesComboBox->GetSelectedItem().Get());
	}

	return NAME_None;
}

void SAddNewGameplayTagWidget::OnCommitNewTagName(const FText& InText, ETextCommit::Type InCommitType)
{
	if (InCommitType == ETextCommit::OnEnter)
	{
		CreateNewGameplayTag();
	}
}

FReply SAddNewGameplayTagWidget::OnAddNewTagButtonPressed()
{
	CreateNewGameplayTag();
	return FReply::Handled();
}

void SAddNewGameplayTagWidget::AddSubtagFromParent(const FString& InParentTagName, const FName& InParentTagSource)
{
	const FText SubtagBaseName = !InParentTagName.IsEmpty() ? FText::Format(FText::FromString(TEXT("{0}.")), FText::FromString(InParentTagName)) : FText();

	SetTagName(SubtagBaseName);
	SelectTagSource(InParentTagSource);

	bShouldGetKeyboardFocus = true;
}

void SAddNewGameplayTagWidget::AddDuplicate(const FString& InParentTagName, const FName& InParentTagSource)
{
	SetTagName(FText::FromString(InParentTagName));
	SelectTagSource(InParentTagSource);

	bShouldGetKeyboardFocus = true;
}

void SAddNewGameplayTagWidget::CreateNewGameplayTag()
{
	if (bRestrictedTags)
	{
		ValidateNewRestrictedTag();
		return;
	}

	if (NotificationItem.IsValid())
	{
		NotificationItem->SetVisibility(EVisibility::Collapsed);
	}

	const UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

	// Only support adding tags via ini file
	if (Manager.ShouldImportTagsFromINI() == false)
	{
		return;
	}

	if (TagSourcesComboBox->GetSelectedItem().Get() == nullptr)
	{
		FNotificationInfo Info(LOCTEXT("NoTagSource", "You must specify a source file for gameplay tags."));
		Info.ExpireDuration = 10.f;
		Info.bUseSuccessFailIcons = true;
		Info.Image = FAppStyle::GetBrush(TEXT("MessageLog.Error"));
		NotificationItem = FSlateNotificationManager::Get().AddNotification(Info);
		
		return;
	}

	const FText TagNameAsText = TagNameTextBox->GetText();
	FString TagName = TagNameAsText.ToString();
	const FString TagComment = TagCommentTextBox->GetText().ToString();
	const FName TagSource = GetSelectedTagSource();

	if (TagName.IsEmpty())
	{
		FNotificationInfo Info(LOCTEXT("NoTagName", "You must specify tag name."));
		Info.ExpireDuration = 10.f;
		Info.bUseSuccessFailIcons = true;
		Info.Image = FAppStyle::GetBrush(TEXT("MessageLog.Error"));
		NotificationItem = FSlateNotificationManager::Get().AddNotification(Info);

		return;
	}

	// check to see if this is a valid tag
	// first check the base rules for all tags then look for any additional rules in the delegate
	FText ErrorMsg;
	if (!UGameplayTagsManager::Get().IsValidGameplayTagString(TagName, &ErrorMsg) || 
		(IsValidTag.IsBound() && !IsValidTag.Execute(TagName, &ErrorMsg))
		)
	{
		FNotificationInfo Info(ErrorMsg);
		Info.ExpireDuration = 10.f;
		Info.bUseSuccessFailIcons = true;
		Info.Image = FAppStyle::GetBrush(TEXT("MessageLog.Error"));
		NotificationItem = FSlateNotificationManager::Get().AddNotification(Info);

		return;
	}

	// set bIsAddingNewTag, this guards against the window closing when it loses focus due to source control checking out a file
	TGuardValue<bool>	Guard(bAddingNewTag, true);

	IGameplayTagsEditorModule::Get().AddNewGameplayTagToINI(TagName, TagComment, TagSource);

	OnGameplayTagAdded.ExecuteIfBound(TagName, TagComment, TagSource);

	Reset(EResetType::DoNotResetSource);
}


void SAddNewGameplayTagWidget::ValidateNewRestrictedTag()
{
	UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

	FString TagName = TagNameTextBox->GetText().ToString();
	FString TagComment = TagCommentTextBox->GetText().ToString();
	const FName TagSource = GetSelectedTagSource();

	if (TagSource == NAME_None)
	{
		FNotificationInfo Info(LOCTEXT("NoRestrictedSource", "You must specify a source file for restricted gameplay tags."));
		Info.ExpireDuration = 10.f;
		Info.bUseSuccessFailIcons = true;
		Info.Image = FAppStyle::GetBrush(TEXT("MessageLog.Error"));

		NotificationItem = FSlateNotificationManager::Get().AddNotification(Info);

		return;
	}

	TArray<FString> TagSourceOwners;
	Manager.GetOwnersForTagSource(TagSource.ToString(), TagSourceOwners);

	bool bHasOwner = false;
	for (const FString& Owner : TagSourceOwners)
	{
		if (!Owner.IsEmpty())
		{
			bHasOwner = true;
			break;
		}
	}

	if (bHasOwner)
	{
		// check if we're one of the owners; if we are then we don't need to pop up the permission dialog
		bool bRequiresPermission = true;
		const FString& UserName = FPlatformProcess::UserName();
		for (const FString& Owner : TagSourceOwners)
		{
			if (Owner.Equals(UserName))
			{
				CreateNewRestrictedGameplayTag();
				bRequiresPermission = false;
			}
		}

		if (bRequiresPermission)
		{
			FString StringToDisplay = TEXT("Do you have permission from ");
			StringToDisplay.Append(TagSourceOwners[0]);
			for (int Idx = 1; Idx < TagSourceOwners.Num(); ++Idx)
			{
				StringToDisplay.Append(TEXT(" or "));
				StringToDisplay.Append(TagSourceOwners[Idx]);
			}
			StringToDisplay.Append(TEXT(" to modify "));
			StringToDisplay.Append(TagSource.ToString());
			StringToDisplay.Append(TEXT("?"));

			FNotificationInfo Info(FText::FromString(StringToDisplay));
			Info.ExpireDuration = 10.f;
			Info.ButtonDetails.Add(FNotificationButtonInfo(LOCTEXT("RestrictedTagPopupButtonAccept", "Yes"), FText(), FSimpleDelegate::CreateSP(this, &SAddNewGameplayTagWidget::CreateNewRestrictedGameplayTag), SNotificationItem::CS_None));
			Info.ButtonDetails.Add(FNotificationButtonInfo(LOCTEXT("RestrictedTagPopupButtonReject", "No"), FText(), FSimpleDelegate::CreateSP(this, &SAddNewGameplayTagWidget::CancelNewTag), SNotificationItem::CS_None));

			NotificationItem = FSlateNotificationManager::Get().AddNotification(Info);
		}
	}
	else
	{
		CreateNewRestrictedGameplayTag();
	}
}

void SAddNewGameplayTagWidget::CreateNewRestrictedGameplayTag()
{
	if (NotificationItem.IsValid())
	{
		NotificationItem->SetVisibility(EVisibility::Collapsed);
	}

	const UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

	// Only support adding tags via ini file
	if (Manager.ShouldImportTagsFromINI() == false)
	{
		return;
	}

	const FString TagName = TagNameTextBox->GetText().ToString();
	const FString TagComment = TagCommentTextBox->GetText().ToString();
	const bool bAllowNonRestrictedChildren = true; // can be changed later
	const FName TagSource = GetSelectedTagSource();

	if (TagName.IsEmpty())
	{
		return;
	}

	// set bIsAddingNewTag, this guards against the window closing when it loses focus due to source control checking out a file
	TGuardValue<bool>	Guard(bAddingNewTag, true);

	IGameplayTagsEditorModule::Get().AddNewGameplayTagToINI(TagName, TagComment, TagSource, true, bAllowNonRestrictedChildren);

	OnGameplayTagAdded.ExecuteIfBound(TagName, TagComment, TagSource);

	Reset(EResetType::DoNotResetSource);
}

void SAddNewGameplayTagWidget::CancelNewTag()
{
	if (NotificationItem.IsValid())
	{
		NotificationItem->SetVisibility(EVisibility::Collapsed);
	}
}

TSharedRef<SWidget> SAddNewGameplayTagWidget::OnGenerateTagSourcesComboBox(TSharedPtr<FString> InItem)
{
	return SNew(STextBlock)
		.Text(FText::AsCultureInvariant(*InItem.Get()));
}

FText SAddNewGameplayTagWidget::CreateTagSourcesComboBoxContent() const
{
	const bool bHasSelectedItem = TagSourcesComboBox.IsValid() && TagSourcesComboBox->GetSelectedItem().IsValid();

	return bHasSelectedItem ? FText::AsCultureInvariant(*TagSourcesComboBox->GetSelectedItem().Get()) : LOCTEXT("NewTagLocationNotSelected", "Not selected");
}

FText SAddNewGameplayTagWidget::CreateTagSourcesComboBoxToolTip() const
{
	const FName ActiveTagSource = GetSelectedTagSource();
	if (!ActiveTagSource.IsNone())
	{
		UGameplayTagsManager& Manager = UGameplayTagsManager::Get();
		const FGameplayTagSource* Source = Manager.FindTagSource(ActiveTagSource);
		if (Source)
		{
			FString FilePath = Source->GetConfigFileName();

			if (FPaths::IsUnderDirectory(FilePath, FPaths::ProjectDir()))
			{
				FPaths::MakePathRelativeTo(FilePath, *FPaths::ProjectDir());
			}
			return FText::FromString(FilePath);
		}
	}

	return FText();
}

#undef LOCTEXT_NAMESPACE
