// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCleanupUnusedGameplayTagsWidget.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "EditorWidgetsModule.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Commands/GenericCommands.h"
#include "GameplayTagsEditorModule.h"
#include "GameplayTagsManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Misc/MessageDialog.h"
#include "Misc/ScopedSlowTask.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "CleanupUnusedGameplayTagsWidget"

namespace SCleanupUnusedGameplayTagsDefs
{
	const FName ColumnID_CheckBoxLabel("CheckBox");
	const FName ColumnID_TagLabel("Tag");
	const FName ColumnID_SourceLabel("Source");
}

/**
 * Represents package item that is displayed as a checkbox inside the package dialog
 */
class FUnusedTagItem : public TSharedFromThis<FUnusedTagItem>
{
public:
	FUnusedTagItem(const TSharedPtr<FGameplayTagNode>& InNode)
		: TagNode(InNode)
		, CheckState(ECheckBoxState::Checked)
	{
	}

	TSharedPtr<FGameplayTagNode> TagNode;
	ECheckBoxState CheckState;
};

class SUnusedTagListRow
	: public SMultiColumnTableRow< TSharedPtr< FUnusedTagItem > >
{

public:
	SLATE_BEGIN_ARGS(SUnusedTagListRow) {}

		/** The list item for this row */
		SLATE_ARGUMENT( TSharedPtr< FUnusedTagItem >, Item )

		/** The list item for this row */
		SLATE_ARGUMENT( TSharedPtr<SListView<TSharedPtr<FUnusedTagItem>>>, List )

	SLATE_END_ARGS()


	/** Construct function for this widget */
	void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView )
	{
		Item = InArgs._Item;
		check(Item.IsValid());

		List = InArgs._List;

		SMultiColumnTableRow< TSharedPtr<FUnusedTagItem> >::Construct(
			FSuperRowType::FArguments()
			, InOwnerTableView);
	}

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the list row. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override
	{
		check(Item.IsValid());

		TSharedPtr<SWidget> ItemContentWidget;
		const FMargin RowPadding(3, 3, 3, 3);

		if (ColumnName == SCleanupUnusedGameplayTagsDefs::ColumnID_CheckBoxLabel)
		{
			ItemContentWidget = SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.Padding(FMargin(10, 3, 6, 3))
				[
					SNew(SCheckBox)
					.IsChecked(this, &SUnusedTagListRow::OnGetDisplayCheckState)
					.OnCheckStateChanged(this, &SUnusedTagListRow::OnDisplayCheckStateChanged)
				];
		}
		else if (ColumnName == SCleanupUnusedGameplayTagsDefs::ColumnID_TagLabel)
		{
			ItemContentWidget = SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.Padding(RowPadding)
				[
					SNew(STextBlock)
					.Text(FText::FromName(Item->TagNode->GetCompleteTagName()))
				];
		}
		else if (ColumnName == SCleanupUnusedGameplayTagsDefs::ColumnID_SourceLabel)
		{
			const TArray<FName>& SourceNames = Item->TagNode->GetAllSourceNames();
			FString SourceString = SourceNames.Num() > 0 ? SourceNames[0].ToString() : FString();
			for (int32 SourceIdx = 1; SourceIdx < SourceNames.Num(); ++SourceIdx)
			{
				SourceString += ", " + SourceNames[SourceIdx].ToString();
			}
			FText SourceText = FText::FromString(SourceString);
			
			ItemContentWidget = SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.Padding(RowPadding)
				[
					SNew(STextBlock)
					.Text(SourceText)
					.ToolTipText(SourceText)
				];
		}

		return ItemContentWidget.ToSharedRef();
	}

	ECheckBoxState OnGetDisplayCheckState() const
	{
		return Item->CheckState;
	}

	void OnDisplayCheckStateChanged(ECheckBoxState InNewState)
	{
		Item->CheckState = InNewState;

		TSharedPtr<SListView<TSharedPtr<FUnusedTagItem>>> UnusedTagsListView = List.Pin();
		if (UnusedTagsListView.IsValid())
		{
			TArray<TSharedPtr<FUnusedTagItem>> SelectedItems = UnusedTagsListView->GetSelectedItems();
			if (SelectedItems.Contains(Item))
			{
				for (const TSharedPtr<FUnusedTagItem>& SelectedItem : SelectedItems)
				{
					SelectedItem->CheckState = InNewState;
				}
			}
		}
	}

private:

	TSharedPtr< FUnusedTagItem > Item;
	TWeakPtr<SListView<TSharedPtr<FUnusedTagItem>>> List;
};

void SCleanupUnusedGameplayTagsWidget::Construct(const FArguments& InArgs)
{
	const FGenericCommands& GenericCommands = FGenericCommands::Get();

	CommandList = MakeShareable(new FUICommandList);

	CommandList->MapAction(
		GenericCommands.Copy,
		FExecuteAction::CreateSP(this, &SCleanupUnusedGameplayTagsWidget::CopySelection));

	TSharedRef< SHeaderRow > HeaderRowWidget = SNew(SHeaderRow);

	HeaderRowWidget->AddColumn(
		SHeaderRow::Column(SCleanupUnusedGameplayTagsDefs::ColumnID_CheckBoxLabel)
		[
			SNew(SBox)
				.Padding(FMargin(6, 3, 6, 3))
				.HAlign(HAlign_Center)
				[
					SNew(SCheckBox)
						.IsChecked(this, &SCleanupUnusedGameplayTagsWidget::GetToggleSelectedState)
						.OnCheckStateChanged(this, &SCleanupUnusedGameplayTagsWidget::OnToggleSelectedCheckBox)
				]
		]
		.FixedWidth(38.0f)
	);

	HeaderRowWidget->AddColumn(
		SHeaderRow::Column(SCleanupUnusedGameplayTagsDefs::ColumnID_TagLabel)
		.DefaultLabel(LOCTEXT("TagColumnLabel", "Tag"))
		.SortMode(this, &SCleanupUnusedGameplayTagsWidget::GetColumnSortMode, SCleanupUnusedGameplayTagsDefs::ColumnID_TagLabel)
		.OnSort(this, &SCleanupUnusedGameplayTagsWidget::OnColumnSortModeChanged)
		.FillWidth(5.0f)
	);

	HeaderRowWidget->AddColumn(
		SHeaderRow::Column(SCleanupUnusedGameplayTagsDefs::ColumnID_SourceLabel)
		.DefaultLabel(LOCTEXT("SourceColumnLabel", "Source"))
		.SortMode(this, &SCleanupUnusedGameplayTagsWidget::GetColumnSortMode, SCleanupUnusedGameplayTagsDefs::ColumnID_SourceLabel)
		.OnSort(this, &SCleanupUnusedGameplayTagsWidget::OnColumnSortModeChanged)
		.FillWidth(5.0f)
	);

	FEditorWidgetsModule& EditorWidgetsModule = FModuleManager::LoadModuleChecked<FEditorWidgetsModule>("EditorWidgets");
	TSharedRef<SWidget> AssetDiscoveryIndicator = EditorWidgetsModule.CreateAssetDiscoveryIndicator(EAssetDiscoveryIndicatorScaleMode::Scale_None, FMargin(16, 8), false);

	ChildSlot
		[
			SNew(SOverlay)

				+ SOverlay::Slot()
				[
					SNew(SVerticalBox)

						+ SVerticalBox::Slot()
						.Padding(5)
						.AutoHeight()
						[
							SNew(STextBlock)
								.Text(this, &SCleanupUnusedGameplayTagsWidget::GetDescriptionText)
						]

						+ SVerticalBox::Slot()
						[
							SAssignNew(UnusedTagsListView, SListView<TSharedPtr<FUnusedTagItem>>)
								.ListItemsSource(&UnusedTags)
								.OnGenerateRow(this, &SCleanupUnusedGameplayTagsWidget::MakeUnusedTagListItemWidget)
								.HeaderRow(HeaderRowWidget)
								.SelectionMode(ESelectionMode::Multi)
						]

						+ SVerticalBox::Slot()
						.Padding(15)
						.AutoHeight()
						.HAlign(HAlign_Right)
						[
							SNew(SHorizontalBox)

								+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(SButton)
										.Text(LOCTEXT("RemoveButton", "Remove Selected Tags"))
										.OnClicked(this, &SCleanupUnusedGameplayTagsWidget::OnRemovePressed)
										.IsEnabled(this, &SCleanupUnusedGameplayTagsWidget::IsRemoveEnabled)
								]
						]
				]

				+ SOverlay::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					AssetDiscoveryIndicator
				]
		];

	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
	if (AssetRegistry.IsLoadingAssets())
	{
		AssetRegistry.OnFilesLoaded().AddSP(this, &SCleanupUnusedGameplayTagsWidget::PopulateUnusedTags);
	}
	else
	{
		PopulateUnusedTags();
	}
}

FReply SCleanupUnusedGameplayTagsWidget::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SCleanupUnusedGameplayTagsWidget::CopySelection()
{
	TArray<TSharedPtr<FUnusedTagItem>> SelectedItems = UnusedTagsListView->GetSelectedItems();
	if (SelectedItems.Num() > 0)
	{
		FString ClipboardString = SelectedItems[0]->TagNode->GetCompleteTagString();
		for (int32 ItemIdx = 1; ItemIdx < SelectedItems.Num(); ++ItemIdx)
		{
			const TSharedPtr<FUnusedTagItem>& Item = SelectedItems[ItemIdx];
			ClipboardString += LINE_TERMINATOR;
			ClipboardString += Item->TagNode->GetCompleteTagString();
		}

		FPlatformApplicationMisc::ClipboardCopy(*ClipboardString);
	}
}

ECheckBoxState SCleanupUnusedGameplayTagsWidget::GetToggleSelectedState() const
{
	if (UnusedTags.Num() == 0)
	{
		return ECheckBoxState::Checked;
	}

	ECheckBoxState CommonCheckState = UnusedTags[0]->CheckState;

	for (int32 ItemIdx = 1; ItemIdx < UnusedTags.Num(); ++ItemIdx)
	{
		if (UnusedTags[ItemIdx]->CheckState != CommonCheckState)
		{
			CommonCheckState = ECheckBoxState::Undetermined;
			break;
		}
	}

	return CommonCheckState;
}

void SCleanupUnusedGameplayTagsWidget::OnToggleSelectedCheckBox(ECheckBoxState InNewState)
{
	for (const TSharedPtr<FUnusedTagItem>& UnusedTag : UnusedTags)
	{
		UnusedTag->CheckState = InNewState;
	}

	UnusedTagsListView->RequestListRefresh();
}

EColumnSortMode::Type SCleanupUnusedGameplayTagsWidget::GetColumnSortMode(const FName ColumnId) const
{
	if (SortByColumn != ColumnId)
	{
		return EColumnSortMode::None;
	}

	return SortMode;
}

void SCleanupUnusedGameplayTagsWidget::OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode)
{
	SortByColumn = ColumnId;
	SortMode = InSortMode;

	SortTags();
}

void SCleanupUnusedGameplayTagsWidget::SortTags()
{
	if (SortByColumn == SCleanupUnusedGameplayTagsDefs::ColumnID_TagLabel)
	{
		if (SortMode == EColumnSortMode::Ascending)
		{
			UnusedTags.Sort([](const TSharedPtr<FUnusedTagItem>& A, const TSharedPtr<FUnusedTagItem>& B) {
				return A->TagNode->GetCompleteTagName().Compare(B->TagNode->GetCompleteTagName()) < 0; });
		}
		else if (SortMode == EColumnSortMode::Descending)
		{
			UnusedTags.Sort([](const TSharedPtr<FUnusedTagItem>& A, const TSharedPtr<FUnusedTagItem>& B) {
				return A->TagNode->GetCompleteTagName().Compare(B->TagNode->GetCompleteTagName()) >= 0; });
		}
	}
	else if (SortByColumn == SCleanupUnusedGameplayTagsDefs::ColumnID_SourceLabel)
	{
		if (SortMode == EColumnSortMode::Ascending)
		{
			UnusedTags.Sort([](const TSharedPtr<FUnusedTagItem>& A, const TSharedPtr<FUnusedTagItem>& B) {
				return A->TagNode->GetFirstSourceName().Compare(B->TagNode->GetFirstSourceName()) < 0; });
		}
		else if (SortMode == EColumnSortMode::Descending)
		{
			UnusedTags.Sort([](const TSharedPtr<FUnusedTagItem>& A, const TSharedPtr<FUnusedTagItem>& B) {
				return A->TagNode->GetFirstSourceName().Compare(B->TagNode->GetFirstSourceName()) >= 0; });
		}
	}

	UnusedTagsListView->RequestListRefresh();
}

FText SCleanupUnusedGameplayTagsWidget::GetDescriptionText() const
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	if (AssetRegistryModule.Get().IsLoadingAssets())
	{
		return LOCTEXT("CleanupDescription_Discovering", "Still discovering assets. Please wait...");
	}
	else if (UnusedTags.Num() > 0)
	{
		return FText::Format(LOCTEXT("CleanupDescription_UnusedTags", "The following {0} explicit gameplay tags are not used by any content."), FText::AsNumber(UnusedTags.Num()));
	}
	else
	{
		return LOCTEXT("CleanupDescription_NoUnused", "All gameplay tags are in use by content.");
	}
}

TSharedRef<ITableRow> SCleanupUnusedGameplayTagsWidget::MakeUnusedTagListItemWidget(TSharedPtr<FUnusedTagItem> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SUnusedTagListRow, OwnerTable)
		.Item(Item)
		.List(UnusedTagsListView);
}

void SCleanupUnusedGameplayTagsWidget::PopulateUnusedTags()
{
	FScopedSlowTask SlowTask(0.f, LOCTEXT("PopulatingUnusedTags", "Populating Unused Tags"));
	SlowTask.MakeDialog();

	TArray<TSharedPtr<FGameplayTagNode>> UnusedTagNodes;
	IGameplayTagsEditorModule::Get().GetUnusedGameplayTags(UnusedTagNodes);

	UnusedTags.Empty(UnusedTagNodes.Num());
	for (const TSharedPtr<FGameplayTagNode>& Node : UnusedTagNodes)
	{
		UnusedTags.Add(MakeShared<FUnusedTagItem>(Node));
	}

	UnusedTagsListView->RequestListRefresh();
}

bool SCleanupUnusedGameplayTagsWidget::IsRemoveEnabled() const
{
	return UnusedTags.Num() > 0;
}

FReply SCleanupUnusedGameplayTagsWidget::OnRemovePressed()
{
	TArray<TSharedPtr<FGameplayTagNode>> TagsToDelete;
	for (const TSharedPtr<FUnusedTagItem>& UnusedTag : UnusedTags)
	{
		if (UnusedTag->CheckState == ECheckBoxState::Checked)
		{
			TagsToDelete.Add(UnusedTag->TagNode);
		}
	}

	UGameplayTagsManager& Manager = UGameplayTagsManager::Get();
	FGameplayTagContainer AllTagsBeforeRemove;
	Manager.RequestAllGameplayTags(AllTagsBeforeRemove, true);

	{
		FScopedSlowTask SlowTask(0.f, LOCTEXT("RemovingTags", "Removing Tags"));
		SlowTask.MakeDialog();
		IGameplayTagsEditorModule::Get().DeleteTagsFromINI(TagsToDelete);

		PopulateUnusedTags();
	}

	FGameplayTagContainer AllTagsAfterRemove;
	Manager.RequestAllGameplayTags(AllTagsAfterRemove, true);
	int32 NumGameplayTagNodesRemoved = AllTagsBeforeRemove.Num() - AllTagsAfterRemove.Num();

	FMessageDialog::Open(EAppMsgType::Ok, FText::Format(LOCTEXT("UnusedTagsRemoved_Text", "{0} gameplay tags were removed in total."), FText::AsNumber(NumGameplayTagNodesRemoved)), LOCTEXT("UnusedTagsRemoved_Title", "Tag Removal Complete"));

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
