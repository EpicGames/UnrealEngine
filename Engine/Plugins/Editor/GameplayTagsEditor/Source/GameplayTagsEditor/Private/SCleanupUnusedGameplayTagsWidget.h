// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Styling/SlateTypes.h"

class FUICommandList;
class FUnusedTagItem;
class STableViewBase;

/** Widget for removing unused gameplay tags */
class SCleanupUnusedGameplayTagsWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCleanupUnusedGameplayTagsWidget) {}
	SLATE_END_ARGS();

	void Construct( const FArguments& InArgs);

private:

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	/** Copies the selected tags to the clipboard */
	void CopySelection();

	/** The state of the column header checkbox that toggles the check state of all items */
	ECheckBoxState GetToggleSelectedState() const;

	/** Handler for when column header checkbox that toggles the check state of all items is clicked */
	void OnToggleSelectedCheckBox(ECheckBoxState InNewState);

	/** The sort mode for the given column */
	EColumnSortMode::Type GetColumnSortMode(const FName ColumnId) const;

	/** Handler for when when a column is clicked to change the sort mode */
	void OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode);

	/** Sorts the tags in the list given the current sorting mode and column */
	void SortTags();

	/** Returns the text at the stop of the dialog */
	FText GetDescriptionText() const;

	/** Creates the widget for a list item */
	TSharedRef<ITableRow> MakeUnusedTagListItemWidget(TSharedPtr<FUnusedTagItem> Item, const TSharedRef<STableViewBase>& OwnerTable);

	/** Updates the list */
	void PopulateUnusedTags();

	/** Returns true if the user can click the remove button */
	bool IsRemoveEnabled() const;

	/** Handler for when the user clicks the button to remove the selected tags */
	FReply OnRemovePressed();

	/** The unused tag items */
	TArray<TSharedPtr<FUnusedTagItem>> UnusedTags;

	/** The list widget for the unused tag items */
	TSharedPtr<SListView<TSharedPtr<FUnusedTagItem>>> UnusedTagsListView;

	/** The command list to handle generic commands like 'Copy' */
	TSharedPtr<FUICommandList> CommandList;

	/** The current sort column */
	FName SortByColumn;

	/** The current sort mode */
	EColumnSortMode::Type SortMode;
};
