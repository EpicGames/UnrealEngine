// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGameplayTagChip.h"
#include "DetailLayoutBuilder.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SWrapBox.h"
#include "GameplayTagStyle.h"
#include "SGameplayTagPicker.h"
#include "Styling/StyleColors.h"
#include "Framework/Application/SlateApplication.h"

#define LOCTEXT_NAMESPACE "GameplayTagChip"

//------------------------------------------------------------------------------
// SGameplayTagChip
//------------------------------------------------------------------------------

SLATE_IMPLEMENT_WIDGET(SGameplayTagChip)
void SGameplayTagChip::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "IsSelected", IsSelectedAttribute, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateLambda([](SWidget& Widget)
			{
				static_cast<SGameplayTagChip&>(Widget).UpdatePillStyle();
			}));

	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "ShowClearButton", ShowClearButtonAttribute, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "Text", TextAttribute, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "ToolTipText", ToolTipTextAttribute, EInvalidateWidgetReason::Layout);
}

SGameplayTagChip::SGameplayTagChip()
	: IsSelectedAttribute(*this)
	, ShowClearButtonAttribute(*this)
	, ToolTipTextAttribute(*this)
	, TextAttribute(*this)
{
}

void SGameplayTagChip::Construct(const FArguments& InArgs)
{
	IsSelectedAttribute.Assign(*this, InArgs._IsSelected);
	ShowClearButtonAttribute.Assign(*this, InArgs._ShowClearButton);
	ToolTipTextAttribute.Assign(*this, InArgs._ToolTipText);
	TextAttribute.Assign(*this, InArgs._Text);
	TextColor = InArgs._TextColor;
	OnClearPressed = InArgs._OnClearPressed;
	OnEditPressed = InArgs._OnEditPressed;
	OnNavigate = InArgs._OnNavigate;
	OnMenu = InArgs._OnMenu;
	bEnableNavigation = InArgs._EnableNavigation;

	TWeakPtr<SGameplayTagChip> WeakSelf = StaticCastWeakPtr<SGameplayTagChip>(AsWeak());

	ChildSlot
	[
		SNew(SBox)
		.HeightOverride(18.0f)
		[
			SNew(SBorder)
			.OnMouseButtonDown_Lambda([WeakSelf](const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
			{
				const TSharedPtr<SGameplayTagChip> Self = WeakSelf.Pin();
				if (Self.IsValid() && MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton))
				{
					if (Self->OnMenu.IsBound())
					{
						return Self->OnMenu.Execute(MouseEvent);
					}
				}
				return FReply::Unhandled();
			})
			.Padding(0.0f)
			.BorderImage(FStyleDefaults::GetNoBrush())
			[
				SNew(SHorizontalBox)
				
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Fill)
				.AutoWidth()
				[
					SAssignNew(ChipButton, SButton)
					.ContentPadding(0)
					.ButtonStyle(FGameplayTagStyle::Get(), "GameplayTags.ChipButton.Selected")
					.ToolTipText_Lambda([WeakSelf]()
					{
						const TSharedPtr<SGameplayTagChip> Self = WeakSelf.Pin();
						return Self.IsValid() ? Self->ToolTipTextAttribute.Get() : FText::GetEmpty();
					})
					.IsEnabled(!InArgs._ReadOnly)
					.OnClicked_Lambda([WeakSelf]()
					{
						if (const TSharedPtr<SGameplayTagChip> Self = WeakSelf.Pin())
						{
							const FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
							if (Self->bEnableNavigation && ModifierKeys.IsControlDown())
							{
								if (Self->OnNavigate.IsBound())
								{
									return Self->OnNavigate.Execute();
								}
							}
								
							if (Self->OnEditPressed.IsBound())
							{
								return Self->OnEditPressed.Execute();
							}
						}							
						return FReply::Unhandled();
					})
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SAssignNew(TagText, STextBlock)
							.Font(FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont")))
							.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
							.Text_Lambda([WeakSelf]()
							{
								const TSharedPtr<SGameplayTagChip> Self = WeakSelf.Pin();
								return Self.IsValid() ? Self->TextAttribute.Get() : FText::GetEmpty();
							})
							.StrikeBrush_Lambda([WeakSelf]()
							{
								if (const TSharedPtr<SGameplayTagChip> Self = WeakSelf.Pin())
								{
									const FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
									if (Self->bEnableNavigation && ModifierKeys.IsControlDown())
									{
										return FGameplayTagStyle::Get().GetBrush("GameplayTags.Chip.Underline");
									}
								}
								return FAppStyle::Get().GetBrush("NoBrush");
							})
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(0)
						[
							SAssignNew(ClearButton, SButton)
							.IsEnabled(!InArgs._ReadOnly)
							.Visibility_Lambda([WeakSelf]()
							{
								const TSharedPtr<SGameplayTagChip> Self = WeakSelf.Pin();
								return (Self.IsValid() && Self->ShowClearButtonAttribute.Get()) ? EVisibility::Visible : EVisibility::Collapsed;
							})
							.VAlign(VAlign_Center)
							.HAlign(HAlign_Center)
							.ButtonStyle(FGameplayTagStyle::Get(), "GameplayTags.ChipClearButton")
							.ToolTipText(LOCTEXT("ClearTag", "Clear Tag"))
							.ContentPadding(FMargin(2, 0, 0, 0))
							.OnClicked_Lambda([WeakSelf]()
							{
								const TSharedPtr<SGameplayTagChip> Self = WeakSelf.Pin();
								if (Self.IsValid() && Self->OnClearPressed.IsBound())
								{
									return Self->OnClearPressed.Execute();
								}
								return FReply::Unhandled();
							})
							[
								SNew(SImage)
								.ColorAndOpacity_Lambda([WeakSelf]()
								{
									const TSharedPtr<SGameplayTagChip> Self = WeakSelf.Pin();
									if (Self.IsValid() && Self->ClearButton.IsValid())
									{
										return Self->ClearButton->IsHovered() ? FStyleColors::White : FStyleColors::Foreground;
									}
									return FStyleColors::Foreground;
								})
								.Image(FAppStyle::GetBrush("Icons.X"))
								.DesiredSizeOverride(FVector2D(12.0f, 12.0f))
							]
						]
					]
				]
			]
		]
	];

	bLastHasIsSelected = true;
	UpdatePillStyle();
}

void SGameplayTagChip::UpdatePillStyle()
{
	const bool bIsSelected = IsSelectedAttribute.Get();
	if (bIsSelected != bLastHasIsSelected)
	{
		if (bIsSelected)
		{
			ChipButton->SetButtonStyle(&FGameplayTagStyle::Get().GetWidgetStyle<FButtonStyle>("GameplayTags.ChipButton.Selected"));
		}
		else
		{
			ChipButton->SetButtonStyle(&FGameplayTagStyle::Get().GetWidgetStyle<FButtonStyle>("GameplayTags.ChipButton.Unselected"));
		}
		bLastHasIsSelected = bIsSelected;
	}
	
	if (TextColor.IsSet())
	{
		TagText->SetColorAndOpacity(TextColor.GetValue());
	}
}

#undef LOCTEXT_NAMESPACE
