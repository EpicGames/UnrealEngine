// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGameplayTagCombo.h"
#include "DetailLayoutBuilder.h"
#include "SGameplayTagPicker.h"
#include "GameplayTagStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "ScopedTransaction.h"
#include "Editor.h"
#include "HAL/PlatformApplicationMisc.h"
#include "GameplayTagEditorUtilities.h"
#include "Widgets/Input/SComboButton.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "GameplayTagCombo"

//------------------------------------------------------------------------------
// SGameplayTagCombo
//------------------------------------------------------------------------------

SLATE_IMPLEMENT_WIDGET(SGameplayTagCombo)
void SGameplayTagCombo::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "Tag", TagAttribute, EInvalidateWidgetReason::Layout);
}

SGameplayTagCombo::SGameplayTagCombo()
	: TagAttribute(*this)
{
}

void SGameplayTagCombo::Construct(const FArguments& InArgs)
{
	TagAttribute.Assign(*this, InArgs._Tag);
	Filter = InArgs._Filter;
	SettingsName = InArgs._SettingsName;
	bIsReadOnly = InArgs._ReadOnly;
	bDisplayShortTagNames = InArgs._DisplayShortTagNames;
	OnTagChanged = InArgs._OnTagChanged;
	PropertyHandle = InArgs._PropertyHandle;

	if (PropertyHandle.IsValid())
	{
		PropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &SGameplayTagCombo::RefreshTagsFromProperty));
		RefreshTagsFromProperty();

		if (Filter.IsEmpty())
		{
			Filter = UGameplayTagsManager::Get().GetCategoriesMetaFromPropertyHandle(PropertyHandle);
		}
		bIsReadOnly = PropertyHandle->IsEditConst();
		bDisplayShortTagNames |= PropertyHandle->GetBoolMetaData("DisplayShortTagNames");
	}
	
	ChildSlot
	[
		SNew(SHorizontalBox) // Extra box to make the combo hug the chip
						
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SAssignNew(ComboButton, SComboButton)
			.ComboButtonStyle(FGameplayTagStyle::Get(), "GameplayTags.ComboButton")
			.HasDownArrow(true)
			.ContentPadding(1)
			.IsEnabled(this, &SGameplayTagCombo::IsValueEnabled)
			.Clipping(EWidgetClipping::OnDemand)
			.OnMenuOpenChanged(this, &SGameplayTagCombo::OnMenuOpenChanged)
			.OnGetMenuContent(this, &SGameplayTagCombo::OnGetMenuContent)
			.ButtonContent()
			[
				SNew(SGameplayTagChip)
				.OnNavigate(InArgs._OnNavigate)
				.OnMenu(InArgs._OnMenu)
				.ShowClearButton(this, &SGameplayTagCombo::ShowClearButton)
				.EnableNavigation(InArgs._EnableNavigation)
				.Text(this, &SGameplayTagCombo::GetText)
				.ToolTipText(this, &SGameplayTagCombo::GetToolTipText)
				.IsSelected(this, &SGameplayTagCombo::IsSelected) 
				.OnClearPressed(this, &SGameplayTagCombo::OnClearPressed)
				.OnEditPressed(this, &SGameplayTagCombo::OnEditTag)
				.OnMenu(this, &SGameplayTagCombo::OnTagMenu)
			]
		]
	];
}

bool SGameplayTagCombo::IsValueEnabled() const
{
	if (PropertyHandle.IsValid())
	{
		return !PropertyHandle->IsEditConst();
	}

	return !bIsReadOnly;
}

FReply SGameplayTagCombo::OnEditTag() const
{
	FReply Reply = FReply::Handled();
	if (ComboButton->ShouldOpenDueToClick())
	{
		ComboButton->SetIsOpen(true);
		if (TagPicker->GetWidgetToFocusOnOpen())
		{
			Reply.SetUserFocus(TagPicker->GetWidgetToFocusOnOpen().ToSharedRef());
		}
	}
	else
	{
		ComboButton->SetIsOpen(false);
	}
	
	return Reply;
}

bool SGameplayTagCombo::ShowClearButton() const
{
	// Show clear button is we have multiple values, or the tag is other than None.
	if (PropertyHandle.IsValid())
	{
		if (bHasMultipleValues)
		{
			return true;
		}
		const FGameplayTag GameplayTag = TagsFromProperty.IsEmpty() ? FGameplayTag() : TagsFromProperty[0]; 
		return GameplayTag.IsValid();
	}
	const FGameplayTag GameplayTag = TagAttribute.Get();
	return GameplayTag.IsValid();
}

FText SGameplayTagCombo::GetText() const
{
	// Pass tag from the properties
	if (PropertyHandle.IsValid())
	{
		if (bHasMultipleValues)
		{
			return LOCTEXT("GameplayTagCombo_MultipleValues", "Multiple Values");
		}
		const FGameplayTag GameplayTag = TagsFromProperty.IsEmpty() ? FGameplayTag() : TagsFromProperty[0];
		if (PropertyHandle->HasMetaData("ShowOnlyLeafTag"))
		{
			return FText::FromName(GameplayTag.GetTagLeafName());
		}

		if (bDisplayShortTagNames)
		{
			FString TagName = GameplayTag.GetTagName().ToString();

			if (TagName.Len() > Filter.Len())
			{
				TagName.RemoveFromStart(Filter + TEXT("."));
			}

			return FText::FromString(MoveTemp(TagName));
		}

		return FText::FromName(GameplayTag.GetTagName());
	}
	return FText::FromName(TagAttribute.Get().GetTagName());
}

FText SGameplayTagCombo::GetToolTipText() const
{
	FGameplayTag TagToDisplay;
	if (PropertyHandle.IsValid())
	{
		TagToDisplay = (TagsFromProperty.Num() > 0) ? TagsFromProperty[0] : FGameplayTag();
	}
	else
	{
		TagToDisplay = TagAttribute.Get();
	}

	TSharedPtr<FGameplayTagNode> TagNode = UGameplayTagsManager::Get().FindTagNode(TagToDisplay);
	if (TagNode.IsValid() && !TagNode->GetDevComment().IsEmpty())
	{
		return FText::FromString(FString::Printf(TEXT("%s\n\n%s"), *TagToDisplay.ToString(), *TagNode->GetDevComment()));
	}
	else
	{
		return FText::FromName(TagToDisplay.GetTagName());
	}
}

bool SGameplayTagCombo::IsSelected() const
{
	// Show in selected state if we have one value and value is valid.
	if (PropertyHandle.IsValid())
	{
		if (bHasMultipleValues)
		{
			return false;
		}
		const FGameplayTag GameplayTag = TagsFromProperty.IsEmpty() ? FGameplayTag() : TagsFromProperty[0]; 
		return GameplayTag.IsValid();
	}
	const FGameplayTag GameplayTag = TagAttribute.Get();
	return GameplayTag.IsValid();
}

FReply SGameplayTagCombo::OnClearPressed()
{
	OnClearTag();
	return FReply::Handled();
}

void SGameplayTagCombo::OnMenuOpenChanged(const bool bOpen) const
{
	if (bOpen && TagPicker.IsValid())
	{
		const FGameplayTag TagToHilight = GetCommonTag();
		TagPicker->RequestScrollToView(TagToHilight);
							
		ComboButton->SetMenuContentWidgetToFocus(TagPicker->GetWidgetToFocusOnOpen());
	}
}

TSharedRef<SWidget> SGameplayTagCombo::OnGetMenuContent()
{
	// If property is not set, well put the edited tag into a container and use that for picking.
	TArray<FGameplayTagContainer> TagContainers;
	if (!PropertyHandle.IsValid())
	{
		const FGameplayTag TagToEdit = TagAttribute.Get();
		TagContainers.Add(FGameplayTagContainer(TagToEdit));
	}

	const bool bIsPickerReadOnly = !IsValueEnabled();
	
	TagPicker = SNew(SGameplayTagPicker)
		.Filter(Filter)
		.SettingsName(SettingsName)
		.ReadOnly(bIsPickerReadOnly)
		.ShowMenuItems(true)
		.MaxHeight(350.0f)
		.MultiSelect(false)
		.OnTagChanged(this, &SGameplayTagCombo::OnTagSelected)
		.Padding(2)
		.PropertyHandle(PropertyHandle)
		.TagContainers(TagContainers);

	if (TagPicker->GetWidgetToFocusOnOpen())
	{
		ComboButton->SetMenuContentWidgetToFocus(TagPicker->GetWidgetToFocusOnOpen());
	}

	return TagPicker.ToSharedRef();
}

void SGameplayTagCombo::OnTagSelected(const TArray<FGameplayTagContainer>& TagContainers)
{
	if (OnTagChanged.IsBound())
	{
		const FGameplayTag NewTag = TagContainers.IsEmpty() ? FGameplayTag() : TagContainers[0].First();
		OnTagChanged.Execute(NewTag);
	}
}

FGameplayTag SGameplayTagCombo::GetCommonTag() const
{
	if (PropertyHandle.IsValid())
	{
		return TagsFromProperty.IsEmpty() ? FGameplayTag() : TagsFromProperty[0]; 
	}
	else
	{
		return TagAttribute.Get();
	}
}

FReply SGameplayTagCombo::OnTagMenu(const FPointerEvent& MouseEvent)
{
	FMenuBuilder MenuBuilder(/*bShouldCloseWindowAfterMenuSelection=*/ true, /*CommandList=*/ nullptr);

	const FGameplayTag GameplayTag = GetCommonTag();
	
	auto IsValidTag = [GameplayTag]()
	{
		return GameplayTag.IsValid();		
	};

	MenuBuilder.AddMenuEntry(
		LOCTEXT("GameplayTagCombo_SearchForReferences", "Search For References"),
		FText::Format(LOCTEXT("GameplayTagCombo_SearchForReferencesTooltip", "Find references to the tag {0}"), FText::AsCultureInvariant(GameplayTag.ToString())),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Search"),
		FUIAction(FExecuteAction::CreateLambda([GameplayTag]()
		{
			// Single tag search
			const FName TagFName = GameplayTag.GetTagName();
			if (FEditorDelegates::OnOpenReferenceViewer.IsBound() && !TagFName.IsNone())
			{
				TArray<FAssetIdentifier> AssetIdentifiers;
				AssetIdentifiers.Emplace(FGameplayTag::StaticStruct(), TagFName);
				FEditorDelegates::OnOpenReferenceViewer.Broadcast(AssetIdentifiers, FReferenceViewerParams());
			}
		}))
		);

	MenuBuilder.AddSeparator();

	MenuBuilder.AddMenuEntry(
	NSLOCTEXT("PropertyView", "CopyProperty", "Copy"),
	FText::Format(LOCTEXT("GameplayTagCombo_CopyTagTooltip", "Copy tag {0} to clipboard"), FText::AsCultureInvariant(GameplayTag.ToString())),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Copy"),
		FUIAction(FExecuteAction::CreateSP(this, &SGameplayTagCombo::OnCopyTag, GameplayTag), FCanExecuteAction::CreateLambda(IsValidTag)));

	MenuBuilder.AddMenuEntry(
	NSLOCTEXT("PropertyView", "PasteProperty", "Paste"),
	LOCTEXT("GameplayTagCombo_PasteTagTooltip", "Paste tags from clipboard."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Paste"),
		FUIAction(FExecuteAction::CreateSP(this, &SGameplayTagCombo::OnPasteTag),FCanExecuteAction::CreateSP(this, &SGameplayTagCombo::CanPaste)));

	MenuBuilder.AddMenuEntry(
	LOCTEXT("GameplayTagCombo_ClearTag", "Clear Gameplay Tag"),
		FText::GetEmpty(),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.X"),
		FUIAction(FExecuteAction::CreateSP(this, &SGameplayTagCombo::OnClearTag), FCanExecuteAction::CreateLambda(IsValidTag)));

	// Spawn context menu
	FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
	FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, MenuBuilder.MakeWidget(), MouseEvent.GetScreenSpacePosition(), FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));

	return FReply::Handled();
}

void SGameplayTagCombo::OnClearTag()
{
	if (PropertyHandle.IsValid())
	{
		FScopedTransaction Transaction(LOCTEXT("GameplayTagCombo_ClearTag", "Clear Gameplay Tag"));
		PropertyHandle->SetValueFromFormattedString(UE::GameplayTags::EditorUtilities::GameplayTagExportText(FGameplayTag()));
	}
				
	OnTagChanged.ExecuteIfBound(FGameplayTag());
}

void SGameplayTagCombo::OnCopyTag(const FGameplayTag TagToCopy) const
{
	// Copy tag as a plain string, GameplayTag's import text can handle that.
	FPlatformApplicationMisc::ClipboardCopy(*TagToCopy.ToString());
}

void SGameplayTagCombo::OnPasteTag()
{
	FString PastedText;
	FPlatformApplicationMisc::ClipboardPaste(PastedText);
	const FGameplayTag PastedTag = UE::GameplayTags::EditorUtilities::GameplayTagTryImportText(PastedText);
	
	if (PastedTag.IsValid())
	{
		if (PropertyHandle.IsValid())
		{
			FScopedTransaction Transaction(LOCTEXT("GameplayTagCombo_PasteTag", "Paste Gameplay Tag"));
			PropertyHandle->SetValueFromFormattedString(PastedText);
			RefreshTagsFromProperty();
		}
		
		OnTagChanged.ExecuteIfBound(PastedTag);
	}
}

bool SGameplayTagCombo::CanPaste() const
{
	FString PastedText;
	FPlatformApplicationMisc::ClipboardPaste(PastedText);
	FGameplayTag PastedTag = UE::GameplayTags::EditorUtilities::GameplayTagTryImportText(PastedText);

	return PastedTag.IsValid();
}

void SGameplayTagCombo::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (!PropertyHandle.IsValid()
		|| !PropertyHandle->IsValidHandle())
	{
		return;
	}

	// Check if cached data has changed, and update it.
	bool bShouldUpdate = false;
	
	TArray<const void*> RawStructData;
	PropertyHandle->AccessRawData(RawStructData);

	if (RawStructData.Num() == TagsFromProperty.Num())
	{
		for (int32 Idx = 0; Idx < RawStructData.Num(); ++Idx)
		{
			if (RawStructData[Idx])
			{
				const FGameplayTag& CurrTag = *(FGameplayTag*)RawStructData[Idx];
				if (CurrTag != TagsFromProperty[Idx])
				{
					bShouldUpdate = true;
					break;
				}
			}
		}
	}

	if (bShouldUpdate)
	{
		RefreshTagsFromProperty();
	}
}

void SGameplayTagCombo::RefreshTagsFromProperty()
{
	if (PropertyHandle.IsValid()
		&& PropertyHandle->IsValidHandle())
	{
		bHasMultipleValues = false;
		TagsFromProperty.Reset();
		
		SGameplayTagPicker::EnumerateEditableTagContainersFromPropertyHandle(PropertyHandle.ToSharedRef(), [this](const FGameplayTagContainer& TagContainer)
		{
			const FGameplayTag TagFromProperty = TagContainer.IsEmpty() ? FGameplayTag() : TagContainer.First(); 
			if (TagsFromProperty.Num() > 0 && TagsFromProperty[0] != TagFromProperty)
			{
				bHasMultipleValues = true;
			}
			TagsFromProperty.Add(TagFromProperty);

			return true;
		});
	}
}

#undef LOCTEXT_NAMESPACE