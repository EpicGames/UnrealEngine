// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGameplayTagContainerCombo.h"
#include "DetailLayoutBuilder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SComboButton.h"
#include "GameplayTagStyle.h"
#include "SGameplayTagPicker.h"
#include "Framework/Application/SlateApplication.h"
#include "ScopedTransaction.h"
#include "Editor.h"
#include "HAL/PlatformApplicationMisc.h"
#include "GameplayTagEditorUtilities.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "GameplayTagContainerCombo"

//------------------------------------------------------------------------------
// SGameplayTagContainerCombo
//------------------------------------------------------------------------------

SLATE_IMPLEMENT_WIDGET(SGameplayTagContainerCombo)
void SGameplayTagContainerCombo::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "TagContainer", TagContainerAttribute, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateLambda([](SWidget& Widget)
			{
				static_cast<SGameplayTagContainerCombo&>(Widget).RefreshTagContainers();
			}));
}

SGameplayTagContainerCombo::SGameplayTagContainerCombo()
	: TagContainerAttribute(*this)
{
}

void SGameplayTagContainerCombo::Construct(const FArguments& InArgs)
{
	Filter = InArgs._Filter;
	SettingsName = InArgs._SettingsName;
	bIsReadOnly = InArgs._ReadOnly;
	bDisplayShortTagNames = InArgs._DisplayShortTagNames;
	OnTagCleared = InArgs._OnTagCleared;
	OnTagContainerComboClosed = InArgs._OnTagContainerComboClosed;
	OnTagContainerChanged = InArgs._OnTagContainerChanged;
	OnGetTagTextColor = InArgs._OnGetTagTextColor;
	PropertyHandle = InArgs._PropertyHandle;

	if (PropertyHandle.IsValid())
	{
		PropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &SGameplayTagContainerCombo::RefreshTagContainers));
		RefreshTagContainers();

		if (Filter.IsEmpty())
		{
			Filter = UGameplayTagsManager::Get().GetCategoriesMetaFromPropertyHandle(PropertyHandle);
		}
		bIsReadOnly = PropertyHandle->IsEditConst();
		bDisplayShortTagNames |= PropertyHandle->GetBoolMetaData("DisplayShortTagNames");
	}
	else
	{
		TagContainerAttribute.Assign(*this, InArgs._TagContainer);
		CachedTagContainers.Add(TagContainerAttribute.Get());
	}

	TWeakPtr<SGameplayTagContainerCombo> WeakSelf = StaticCastWeakPtr<SGameplayTagContainerCombo>(AsWeak());

	TagListView = SNew(SListView<TSharedPtr<FEditableItem>>)
		.ListItemsSource(&TagsToEdit)
		.SelectionMode(ESelectionMode::None)
		.ListViewStyle(&FAppStyle::Get().GetWidgetStyle<FTableViewStyle>("SimpleListView"))
		.OnGenerateRow(this, &SGameplayTagContainerCombo::MakeTagListViewRow)
		.Visibility_Lambda([WeakSelf]()
		{
			if (const TSharedPtr<SGameplayTagContainerCombo> Self = WeakSelf.Pin())
			{
				return Self->TagsToEdit.Num() > 0 ? EVisibility::Visible : EVisibility::Collapsed;
			}
			return EVisibility::Collapsed;
		});

	ChildSlot
	[
		SNew(SHorizontalBox)
			
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Top)
		[
			SAssignNew(ComboButton, SComboButton)
			.ComboButtonStyle(FGameplayTagStyle::Get(), "GameplayTagsContainer.ComboButton")
			.IsEnabled(this, &SGameplayTagContainerCombo::IsValueEnabled)
			.HasDownArrow(true)
			.VAlign(VAlign_Top)
			.ContentPadding(0)
			.OnMenuOpenChanged(this, &SGameplayTagContainerCombo::OnMenuOpenChanged)
			.OnGetMenuContent(this, &SGameplayTagContainerCombo::OnGetMenuContent)
			.ButtonContent()
			[
				SNew(SHorizontalBox)
				
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Top)
				[
					SNew(SBorder)
					.Padding(FMargin(6,2))
					.BorderImage(FGameplayTagStyle::GetBrush("GameplayTags.Container"))
					.OnMouseButtonDown_Lambda([WeakSelf](const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
					{
						const TSharedPtr<SGameplayTagContainerCombo> Self = WeakSelf.Pin();
						if (Self.IsValid() && MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton) && Self->TagsToEdit.Num() <= 0)
						{
							return Self->OnEmptyMenu(MouseEvent);
						}
						return FReply::Unhandled();
					})
					[
						SNew(SHorizontalBox)

						// Tag list
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Top)
						.AutoWidth()
						[
							TagListView.ToSharedRef()
						]
						
						// Empty indicator
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						.Padding(FMargin(4, 2))
						[
							SNew(SBox)
							.HeightOverride(18.0f) // Same is SGameplayTagChip height
							.VAlign(VAlign_Center)
							.Padding(0, 0, 8, 0)
							.Visibility_Lambda([WeakSelf]()
							{
								if (const TSharedPtr<SGameplayTagContainerCombo> Self = WeakSelf.Pin())
								{
									return Self->TagsToEdit.Num() > 0 ? EVisibility::Collapsed : EVisibility::Visible;
								}
								return EVisibility::Collapsed;
							})
							[
								SNew(STextBlock)
								.ColorAndOpacity(FSlateColor::UseSubduedForeground())
								.Font(FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont")))
								.Text(LOCTEXT("GameplayTagContainerCombo_Empty", "Empty"))
								.ToolTipText(LOCTEXT("GameplayTagContainerCombo_EmptyTooltip", "Empty Gameplay Tag container"))
							]
						]
					]
				]
			]
		]
	];
}

bool SGameplayTagContainerCombo::IsValueEnabled() const
{
	if (PropertyHandle.IsValid())
	{
		return !PropertyHandle->IsEditConst();
	}

	return !bIsReadOnly;
}

TSharedRef<ITableRow> SGameplayTagContainerCombo::MakeTagListViewRow(TSharedPtr<FEditableItem> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	FString TagName = Item->Tag.GetTagName().ToString();
	if (bDisplayShortTagNames)
	{
		if (TagName.Len() > Filter.Len())
		{
			TagName.RemoveFromStart(Filter + TEXT("."));
		}
	}
	
	TOptional<FSlateColor> TextColor;
	if (OnGetTagTextColor.IsBound())
	{
		TextColor = OnGetTagTextColor.Execute(Item->Tag);
	}

	return SNew(STableRow<TSharedPtr<FEditableItem>>, OwnerTable)
		.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SimpleTableView.Row"))
		.Padding(FMargin(0,2))
		[
			SNew(SGameplayTagChip)
			.ReadOnly(bIsReadOnly)
			.ShowClearButton(true)
			.Text(FText::FromString(TagName))
			.TextColor(TextColor)
			.ToolTipText(this, &SGameplayTagContainerCombo::GetTagToolTipText, Item->Tag)
			.IsSelected(!Item->bMultipleValues)
			.OnClearPressed(this, &SGameplayTagContainerCombo::OnClearTagClicked, Item->Tag)
			.OnEditPressed(this, &SGameplayTagContainerCombo::OnEditClicked, Item->Tag)
			.OnMenu(this, &SGameplayTagContainerCombo::OnTagMenu, Item->Tag)
		];
}

void SGameplayTagContainerCombo::OnMenuOpenChanged(const bool bOpen)
{
	if (bOpen && TagPicker.IsValid())
	{
		if (!TagToHilight.IsValid() && TagsToEdit.Num() > 0)
		{
			TagToHilight = TagsToEdit[0]->Tag;
		}
		TagPicker->RequestScrollToView(TagToHilight);
	}
	else
	{
		OnTagContainerComboClosed.ExecuteIfBound(CachedTagContainers[0]);
	}
	// Reset tag to hilight
	TagToHilight = FGameplayTag();
}

TSharedRef<SWidget> SGameplayTagContainerCombo::OnGetMenuContent()
{
	// If property is not set, we'll put the edited tag from attribute into a container and use that for picking.
	TArray<FGameplayTagContainer> TagContainersToEdit;
	if (!PropertyHandle.IsValid() && CachedTagContainers.Num() == 1)
	{
		CachedTagContainers[0] = TagContainerAttribute.Get();
		TagContainersToEdit.Add(CachedTagContainers[0]);
	}

	const bool bIsPickerReadOnly = !IsValueEnabled();

	TagPicker = SNew(SGameplayTagPicker)
		.Filter(Filter)
		.SettingsName(SettingsName)
		.ReadOnly(bIsPickerReadOnly)
		.ShowMenuItems(true)
		.MaxHeight(400.0f)
		.MultiSelect(true)
		.OnTagChanged(this, &SGameplayTagContainerCombo::OnTagChanged)
		.Padding(FMargin(2,0,2,0))
		.PropertyHandle(PropertyHandle)
		.TagContainers(TagContainersToEdit);

	if (TagPicker->GetWidgetToFocusOnOpen())
	{
		ComboButton->SetMenuContentWidgetToFocus(TagPicker->GetWidgetToFocusOnOpen());
	}

	return TagPicker.ToSharedRef();
}

FText SGameplayTagContainerCombo::GetTagToolTipText(const FGameplayTag TagToDisplay) const
{
	TSharedPtr<FGameplayTagNode> TagNode = UGameplayTagsManager::Get().FindTagNode(TagToDisplay);

	if (TagNode.IsValid() && !TagNode->GetDevComment().IsEmpty())
	{
		return FText::FromString(FString::Printf(TEXT("%s\n\n%s"), *TagToDisplay.ToString(), *TagNode->GetDevComment()));
	}
	else
	{
		return FText::FromName(TagToDisplay.GetTagName());
	}
}

FReply SGameplayTagContainerCombo::OnTagMenu(const FPointerEvent& MouseEvent, const FGameplayTag GameplayTag)
{
	FMenuBuilder MenuBuilder(/*bShouldCloseWindowAfterMenuSelection=*/ true, /*CommandList=*/ nullptr);

	TWeakPtr<SGameplayTagContainerCombo> WeakSelf = StaticCastWeakPtr<SGameplayTagContainerCombo>(AsWeak());

	auto IsValidTag = [GameplayTag]()
	{
		return GameplayTag.IsValid();		
	};
	
	MenuBuilder.AddMenuEntry(
		LOCTEXT("GameplayTagContainerCombo_SearchForReferences", "Search For References"),
		FText::Format(LOCTEXT("GameplayTagContainerCombo_SearchForReferencesTooltip", "Find references to the tag {0}"), FText::AsCultureInvariant(GameplayTag.ToString())),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Search"),
		FUIAction(FExecuteAction::CreateLambda([GameplayTag]()
			{
				// Single tag search
				const FName TagFName = GameplayTag.GetTagName();
				if (FEditorDelegates::OnOpenReferenceViewer.IsBound() && !TagFName.IsNone())
				{
					TArray<FAssetIdentifier> AssetIdentifiers;
					AssetIdentifiers.Emplace(FGameplayTag::StaticStruct(), TagFName);
					FEditorDelegates::OnOpenReferenceViewer.Broadcast(AssetIdentifiers, FReferenceViewerParams());
				}
			}),
			FCanExecuteAction::CreateLambda(IsValidTag))
		);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("GameplayTagContainerCombo_WholeContainerSearchForReferences", "Search For Any References"),
		LOCTEXT("GameplayTagContainerCombo_WholeContainerSearchForReferencesTooltip", "Find referencers that reference *any* of the tags in this container"),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateSP(this, &SGameplayTagContainerCombo::OnSearchForAnyReferences), FCanExecuteAction::CreateLambda(IsValidTag)));

	MenuBuilder.AddSeparator();

	MenuBuilder.AddMenuEntry(
	NSLOCTEXT("PropertyView", "CopyProperty", "Copy"),
		FText::Format(LOCTEXT("GameplayTagContainerCombo_CopyTagTooltip", "Copy tag {0} to clipboard"), FText::AsCultureInvariant(GameplayTag.ToString())),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Copy"),
		FUIAction(FExecuteAction::CreateSP(this, &SGameplayTagContainerCombo::OnCopyTag, GameplayTag), FCanExecuteAction::CreateLambda(IsValidTag)));
	
	MenuBuilder.AddMenuEntry(
	NSLOCTEXT("PropertyView", "PasteProperty", "Paste"),
		LOCTEXT("GameplayTagContainerCombo_PasteTagTooltip", "Paste tags from clipboard."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Paste"),
		FUIAction(FExecuteAction::CreateSP(this, &SGameplayTagContainerCombo::OnPasteTag), FCanExecuteAction::CreateSP(this, &SGameplayTagContainerCombo::CanPaste)));
	
	MenuBuilder.AddMenuEntry(
	LOCTEXT("GameplayTagContainerCombo_ClearAll", "Clear All Tags"),
		FText::GetEmpty(),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.X"),
		FUIAction(FExecuteAction::CreateSP(this, &SGameplayTagContainerCombo::OnClearAll)));

	// Spawn context menu
	FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
	FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, MenuBuilder.MakeWidget(), MouseEvent.GetScreenSpacePosition(), FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));

	return FReply::Handled();

}

FReply SGameplayTagContainerCombo::OnEmptyMenu(const FPointerEvent& MouseEvent)
{
	FMenuBuilder MenuBuilder(/*bShouldCloseWindowAfterMenuSelection=*/ true, /*CommandList=*/ nullptr);
	
	MenuBuilder.AddMenuEntry(
	NSLOCTEXT("PropertyView", "PasteProperty", "Paste"),
		LOCTEXT("GameplayTagContainerCombo_PasteTagTooltip", "Paste tags from clipboard."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Paste"),
		FUIAction(FExecuteAction::CreateSP(this, &SGameplayTagContainerCombo::OnPasteTag), FCanExecuteAction::CreateSP(this, &SGameplayTagContainerCombo::CanPaste)));

	// Spawn context menu
	const FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
	FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, MenuBuilder.MakeWidget(), MouseEvent.GetScreenSpacePosition(), FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
	
	return FReply::Handled();
}

void SGameplayTagContainerCombo::OnSearchForAnyReferences() const
{
	if (FEditorDelegates::OnOpenReferenceViewer.IsBound()
		&& TagsToEdit.Num() > 0)
	{
		TArray<FAssetIdentifier> AssetIdentifiers;
		AssetIdentifiers.Reserve(TagsToEdit.Num());
		for (const TSharedPtr<FEditableItem>& Item : TagsToEdit)
		{
			check(Item.IsValid());
			const FName TagFName = Item->Tag.GetTagName();
			if (!TagFName.IsNone())
			{
				AssetIdentifiers.Emplace(FGameplayTag::StaticStruct(), TagFName);
			}
		}
		FEditorDelegates::OnOpenReferenceViewer.Broadcast(AssetIdentifiers, FReferenceViewerParams());
	}
}

bool SGameplayTagContainerCombo::IsOpen() const
{
	return ComboButton.IsValid() && ComboButton->IsOpen();
}

void SGameplayTagContainerCombo::Open(const FGameplayTag TagToFocus)
{
	TagToHilight = TagToFocus;
	ComboButton->SetIsOpen(true);
}

FReply SGameplayTagContainerCombo::OnEditClicked(const FGameplayTag InTagToHilight)
{
	FReply Reply = FReply::Handled();
	if (ComboButton->ShouldOpenDueToClick())
	{
		Open(InTagToHilight);
		
		if (TagPicker.IsValid() && TagPicker->GetWidgetToFocusOnOpen())
		{
			Reply.SetUserFocus(TagPicker->GetWidgetToFocusOnOpen().ToSharedRef());
		}
	}
	else
	{
		ComboButton->SetIsOpen(false);
	}
	
	return Reply;
}

FReply SGameplayTagContainerCombo::OnClearAllClicked()
{
	OnClearAll();
	return FReply::Handled();
}

void SGameplayTagContainerCombo::OnTagChanged(const TArray<FGameplayTagContainer>& TagContainers)
{
	// Property is handled in the picker.

	// Update for attribute version and callbacks.
	CachedTagContainers = TagContainers;
	
	if (!TagContainers.IsEmpty())
	{
		OnTagContainerChanged.ExecuteIfBound(TagContainers[0]);
	}

	RefreshTagContainers();
}

void SGameplayTagContainerCombo::OnClearAll()
{
	if (PropertyHandle.IsValid())
	{
		FScopedTransaction Transaction(LOCTEXT("GameplayTagContainerCombo_ClearAll", "Clear All Tags"));
		PropertyHandle->SetValueFromFormattedString(FGameplayTagContainer().ToString());
	}

	// Update for attribute version and callbacks.
	for (FGameplayTagContainer& TagContainer : CachedTagContainers)
	{
		TagContainer.Reset();
	}

	if (!CachedTagContainers.IsEmpty())
	{
		OnTagContainerChanged.ExecuteIfBound(CachedTagContainers[0]);
	}

	RefreshTagContainers();
}

void SGameplayTagContainerCombo::OnCopyTag(const FGameplayTag TagToCopy) const
{
	// Copy tag as a plain string, GameplayTag's import text can handle that.
	FPlatformApplicationMisc::ClipboardCopy(*TagToCopy.ToString());
}

void SGameplayTagContainerCombo::OnPasteTag()
{
	FString PastedText;
	FPlatformApplicationMisc::ClipboardPaste(PastedText);
	bool bHandled = false;

	// Try to paste single tag
	const FGameplayTag PastedTag = UE::GameplayTags::EditorUtilities::GameplayTagTryImportText(PastedText);
	if (PastedTag.IsValid())
	{
		if (PropertyHandle.IsValid())
		{
			// From property
			TArray<FString> NewValues;
			SGameplayTagPicker::EnumerateEditableTagContainersFromPropertyHandle(PropertyHandle.ToSharedRef(), [&NewValues, PastedTag](const FGameplayTagContainer& EditableTagContainer)
			{
				FGameplayTagContainer TagContainerCopy = EditableTagContainer;
				TagContainerCopy.AddTag(PastedTag);

				NewValues.Add(TagContainerCopy.ToString());
				return true;
			});

			FScopedTransaction Transaction(LOCTEXT("GameplayTagContainerCombo_PasteGameplayTag", "Paste Gameplay Tag"));
			PropertyHandle->SetPerObjectValues(NewValues);
		}

		// Update for attribute version and callbacks.
		for (FGameplayTagContainer& TagContainer : CachedTagContainers)
		{
			TagContainer.AddTag(PastedTag);
		}

		bHandled = true;
	}

	// Try to paste a container
	if (!bHandled)
	{
		const FGameplayTagContainer PastedTagContainer = UE::GameplayTags::EditorUtilities::GameplayTagContainerTryImportText(PastedText);
		if (PastedTagContainer.IsValid())
		{
			if (PropertyHandle.IsValid())
			{
				// From property
				FScopedTransaction Transaction(LOCTEXT("GameplayTagContainerCombo_PasteGameplayTagContainer", "Paste Gameplay Tag Container"));
				PropertyHandle->SetValueFromFormattedString(PastedText);
			}

			// Update for attribute version and callbacks.
			for (FGameplayTagContainer& TagContainer : CachedTagContainers)
			{
				TagContainer = PastedTagContainer;
			}
			
			bHandled = true;
		}
	}

	if (bHandled)
	{
		if (!CachedTagContainers.IsEmpty())
		{
			OnTagContainerChanged.ExecuteIfBound(CachedTagContainers[0]);
		}

		RefreshTagContainers();
	}
}

bool SGameplayTagContainerCombo::CanPaste() const
{
	FString PastedText;
	FPlatformApplicationMisc::ClipboardPaste(PastedText);
	
	const FGameplayTag PastedTag = UE::GameplayTags::EditorUtilities::GameplayTagTryImportText(PastedText);
	if (PastedTag.IsValid())
	{
		return true;
	}

	const FGameplayTagContainer PastedTagContainer = UE::GameplayTags::EditorUtilities::GameplayTagContainerTryImportText(PastedText);
	if (PastedTagContainer.IsValid())
	{
		return true;
	}

	return false;
}
	
FReply SGameplayTagContainerCombo::OnClearTagClicked(const FGameplayTag TagToClear)
{
	if (PropertyHandle.IsValid())
	{
		// From property
		TArray<FString> NewValues;
		SGameplayTagPicker::EnumerateEditableTagContainersFromPropertyHandle(PropertyHandle.ToSharedRef(), [&NewValues, TagToClear](const FGameplayTagContainer& EditableTagContainer)
		{
			FGameplayTagContainer TagContainerCopy = EditableTagContainer;
			TagContainerCopy.RemoveTag(TagToClear);

			NewValues.Add(TagContainerCopy.ToString());
			return true;
		});

		FScopedTransaction Transaction(LOCTEXT("GameplayTagContainerCombo_Remove", "Remove Gameplay Tag"));
		PropertyHandle->SetPerObjectValues(NewValues);
	}
	
	// Update for attribute version and callbacks.
	for (FGameplayTagContainer& TagContainer : CachedTagContainers)
	{
		TagContainer.RemoveTag(TagToClear);
	}

	if (!CachedTagContainers.IsEmpty())
	{
		OnTagContainerChanged.ExecuteIfBound(CachedTagContainers[0]);
	}

	RefreshTagContainers();
	OnTagCleared.ExecuteIfBound();

	return FReply::Handled();
}

void SGameplayTagContainerCombo::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (!PropertyHandle.IsValid()
		|| !PropertyHandle->IsValidHandle())
	{
		return;
	}

	// Check if cached data has changed, and update it.
	bool bShouldUpdate = false;
		
	TArray<const void*> RawStructData;
	PropertyHandle->AccessRawData(RawStructData);

	if (RawStructData.Num() == CachedTagContainers.Num())
	{
		for (int32 Idx = 0; Idx < RawStructData.Num(); ++Idx)
		{
			if (RawStructData[Idx])
			{
				const FGameplayTagContainer& TagContainer = *(FGameplayTagContainer*)RawStructData[Idx];
				if (TagContainer != CachedTagContainers[Idx])
				{
					bShouldUpdate = true;
					break;
				}
			}
		}
	}

	if (bShouldUpdate)
	{
		RefreshTagContainers();
	}
}

void SGameplayTagContainerCombo::RefreshTagContainers()
{
	CachedTagContainers.Reset();
	TagsToEdit.Reset();

	if (PropertyHandle.IsValid())
	{
		if (PropertyHandle->IsValidHandle())
		{
			// From property
			SGameplayTagPicker::EnumerateEditableTagContainersFromPropertyHandle(PropertyHandle.ToSharedRef(), [this](const FGameplayTagContainer& InTagContainer)
			{
				CachedTagContainers.Add(InTagContainer);

				for (auto It = InTagContainer.CreateConstIterator(); It; ++It)
				{
					const FGameplayTag Tag = *It;
					const int32 ExistingItemIndex = TagsToEdit.IndexOfByPredicate([Tag](const TSharedPtr<FEditableItem>& Item)
					{
						return Item.IsValid() && Item->Tag == Tag;
					});
					if (ExistingItemIndex != INDEX_NONE)
					{
						TagsToEdit[ExistingItemIndex]->Count++;
					}
					else
					{
						TagsToEdit.Add(MakeShared<FEditableItem>(Tag));
					}
				}
				
				return true;
			});
		}
	}
	else
	{
		// From attribute
		const FGameplayTagContainer& InTagContainer = TagContainerAttribute.Get(); 
		
		CachedTagContainers.Add(InTagContainer);

		for (auto It = InTagContainer.CreateConstIterator(); It; ++It)
		{
			TagsToEdit.Add(MakeShared<FEditableItem>(*It));
		}
	}

	const int32 PropertyCount = CachedTagContainers.Num();
	for (TSharedPtr<FEditableItem>& Item : TagsToEdit)
	{
		check(Item.IsValid());
		if (Item->Count != PropertyCount)
		{
			Item->bMultipleValues = true;
		}
	}
	
	TagsToEdit.StableSort([](const TSharedPtr<FEditableItem>& LHS, const TSharedPtr<FEditableItem>& RHS)
	{
		check(LHS.IsValid() && RHS.IsValid());
		return LHS->Tag < RHS->Tag;
	});

	// Refresh the slate list
	if (TagListView.IsValid())
	{
		TagListView->SetItemsSource(&TagsToEdit);
		TagListView->RequestListRefresh();
	}
}

#undef LOCTEXT_NAMESPACE