// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGameplayTagContainerGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "GameplayTagContainer.h"
#include "SGameplayTagContainerCombo.h"
#include "GameplayTagEditorUtilities.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "GameplayTagContainerGraphPin"

void SGameplayTagContainerGraphPin::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

void SGameplayTagContainerGraphPin::ParseDefaultValueData()
{
	// Read using import text, but with serialize flag set so it doesn't always throw away invalid ones
	GameplayTagContainer.FromExportString(GraphPinObj->GetDefaultAsString(), PPF_SerializedAsImportText);
}

void SGameplayTagContainerGraphPin::OnTagContainerChanged(const FGameplayTagContainer& NewTagContainer)
{
	GameplayTagContainer = NewTagContainer;

	const FString TagContainerString = UE::GameplayTags::EditorUtilities::GameplayTagContainerExportText(GameplayTagContainer);

	FString CurrentDefaultValue = GraphPinObj->GetDefaultAsString();
	if (CurrentDefaultValue.IsEmpty())
	{
		CurrentDefaultValue = UE::GameplayTags::EditorUtilities::GameplayTagContainerExportText(FGameplayTagContainer());
	}
			
	if (!CurrentDefaultValue.Equals(TagContainerString))
	{
		const FScopedTransaction Transaction(LOCTEXT("ChangeDefaultValue", "Change Pin Default Value"));
		GraphPinObj->Modify();
		GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, TagContainerString);
	}
}

TSharedRef<SWidget> SGameplayTagContainerGraphPin::GetDefaultValueWidget()
{
	if (GraphPinObj == nullptr)
	{
		return SNullWidget::NullWidget;
	}

	ParseDefaultValueData();
	const FString FilterString = UE::GameplayTags::EditorUtilities::ExtractTagFilterStringFromGraphPin(GraphPinObj);

	return SNew(SGameplayTagContainerCombo)
		.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
		.Filter(FilterString)
		.TagContainer(this, &SGameplayTagContainerGraphPin::GetTagContainer)
		.OnTagContainerChanged(this, &SGameplayTagContainerGraphPin::OnTagContainerChanged);
}

FGameplayTagContainer SGameplayTagContainerGraphPin::GetTagContainer() const
{
	return GameplayTagContainer;
}

#undef LOCTEXT_NAMESPACE
