// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "KismetPins/SGraphPinStructInstance.h"
#include "GameplayTagContainer.h"

/** Almost the same as a tag pin, but supports multiple tags */
class SGameplayTagContainerGraphPin : public SGraphPinStructInstance
{
public:
	SLATE_BEGIN_ARGS(SGameplayTagContainerGraphPin) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

protected:
	//~ Begin SGameplayTagGraphPin Interface
	virtual void ParseDefaultValueData() override;
	virtual TSharedRef<SWidget> GetDefaultValueWidget() override;
	//~ End SGraphPin Interface

	FGameplayTagContainer GetTagContainer() const;
	void OnTagContainerChanged(const FGameplayTagContainer& NewTagContainer);
	
	FGameplayTagContainer GameplayTagContainer;
};
