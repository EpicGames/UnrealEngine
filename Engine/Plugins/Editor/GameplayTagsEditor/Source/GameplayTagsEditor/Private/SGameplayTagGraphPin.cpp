// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGameplayTagGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "GameplayTagContainer.h"
#include "GameplayTagEditorUtilities.h"
#include "SGameplayTagCombo.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "GameplayTagGraphPin"

void SGameplayTagGraphPin::Construct( const FArguments& InArgs, UEdGraphPin* InGraphPinObj )
{
	SGraphPin::Construct( SGraphPin::FArguments(), InGraphPinObj );
}

void SGameplayTagGraphPin::ParseDefaultValueData()
{
	// Read using import text, but with serialize flag set so it doesn't always throw away invalid ones
	GameplayTag.FromExportString(GraphPinObj->GetDefaultAsString(), PPF_SerializedAsImportText);
}

void SGameplayTagGraphPin::OnTagChanged(const FGameplayTag NewTag)
{
	GameplayTag = NewTag;

	const FString TagString = UE::GameplayTags::EditorUtilities::GameplayTagExportText(GameplayTag);

	FString CurrentDefaultValue = GraphPinObj->GetDefaultAsString();
	if (CurrentDefaultValue.IsEmpty())
	{
		CurrentDefaultValue = UE::GameplayTags::EditorUtilities::GameplayTagExportText(FGameplayTag());
	}
			
	if (!CurrentDefaultValue.Equals(TagString))
	{
		const FScopedTransaction Transaction(LOCTEXT("ChangeDefaultValue", "Change Pin Default Value"));
		GraphPinObj->Modify();
		GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, TagString);
	}
}

TSharedRef<SWidget> SGameplayTagGraphPin::GetDefaultValueWidget()
{
	if (GraphPinObj == nullptr)
	{
		return SNullWidget::NullWidget;
	}

	ParseDefaultValueData();
	const FString FilterString = UE::GameplayTags::EditorUtilities::ExtractTagFilterStringFromGraphPin(GraphPinObj);

	return SNew(SGameplayTagCombo)
		.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
		.Filter(FilterString)
		.Tag(this, &SGameplayTagGraphPin::GetGameplayTag)
		.OnTagChanged(this, &SGameplayTagGraphPin::OnTagChanged);
}

FGameplayTag SGameplayTagGraphPin::GetGameplayTag() const
{
	return GameplayTag;
}

#undef LOCTEXT_NAMESPACE
