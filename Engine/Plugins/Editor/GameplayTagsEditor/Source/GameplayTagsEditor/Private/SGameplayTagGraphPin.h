// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "KismetPins/SGraphPinStructInstance.h"
#include "GameplayTagContainer.h"

template <typename ItemType> class SListView;

/** Pin that represents a single gameplay tag, overrides the generic struct widget because tags have their own system for saving changes */
class SGameplayTagGraphPin : public SGraphPinStructInstance
{
public:
	SLATE_BEGIN_ARGS(SGameplayTagGraphPin) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

protected:
	//~ Begin SGraphPinStructInstance Interface
	virtual void ParseDefaultValueData() override;
	virtual TSharedRef<SWidget> GetDefaultValueWidget() override;
	//~ End SGraphPin Interface

	FGameplayTag GetGameplayTag() const;
	void OnTagChanged(const FGameplayTag NewTag);

	FGameplayTag GameplayTag;
};
