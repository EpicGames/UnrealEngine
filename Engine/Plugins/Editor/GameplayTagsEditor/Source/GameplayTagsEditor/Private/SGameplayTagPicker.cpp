// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGameplayTagPicker.h"
#include "Misc/ConfigCacheIni.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "Styling/AppStyle.h"
#include "Widgets/SWindow.h"
#include "Misc/MessageDialog.h"
#include "GameplayTagsModule.h"
#include "ScopedTransaction.h"
#include "Textures/SlateIcon.h"
#include "PropertyHandle.h"
#include "Widgets/Input/SSearchBox.h"
#include "GameplayTagsEditorModule.h"
#include "GameplayTagsModule.h"
#include "Framework/Application/SlateApplication.h"
#include "SAddNewGameplayTagWidget.h"
#include "SRenameGameplayTagDialog.h"
#include "AssetRegistry/AssetData.h"
#include "Editor.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Docking/TabManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Misc/EnumerateRange.h"
#include "Styling/StyleColors.h"

#define LOCTEXT_NAMESPACE "GameplayTagPicker"

const FString SGameplayTagPicker::SettingsIniSection = TEXT("GameplayTagPicker");

bool SGameplayTagPicker::EnumerateEditableTagContainersFromPropertyHandle(const TSharedRef<IPropertyHandle>& PropHandle, TFunctionRef<bool(const FGameplayTagContainer&)> Callback)
{
	const FStructProperty* StructProperty = CastField<FStructProperty>(PropHandle->GetProperty());
	if (StructProperty && StructProperty->Struct->IsChildOf(FGameplayTagContainer::StaticStruct()))
	{
		PropHandle->EnumerateRawData([&Callback](void* RawData, const int32 /*DataIndex*/, const int32 /*NumDatas*/)
		{
			// Report empty container even if the instance data is null to match all the instances indices of the property handle.
			if (RawData)
			{
				return Callback(*static_cast<FGameplayTagContainer*>(RawData));
			}
			return Callback(FGameplayTagContainer());
		});
		return true;
	}
	if (StructProperty && StructProperty->Struct->IsChildOf(FGameplayTag::StaticStruct()))
	{
		PropHandle->EnumerateRawData([&Callback](void* RawData, const int32 /*DataIndex*/, const int32 /*NumDatas*/)
		{
			// Report empty container even if the instance data is null to match all the instances indices of the property handle.
			FGameplayTagContainer Container;
			if (RawData)
			{
				Container.AddTag(*static_cast<FGameplayTag*>(RawData));
			}
			return Callback(Container);
		});
		return true;
	}
	return false;
}

bool SGameplayTagPicker::GetEditableTagContainersFromPropertyHandle(const TSharedRef<IPropertyHandle>& PropHandle, TArray<FGameplayTagContainer>& OutEditableTagContainers)
{
	OutEditableTagContainers.Reset();
	return EnumerateEditableTagContainersFromPropertyHandle(PropHandle, [&OutEditableTagContainers](const FGameplayTagContainer& EditableTagContainer)
	{
		OutEditableTagContainers.Add(EditableTagContainer);
		return true;
	});
}

SGameplayTagPicker::~SGameplayTagPicker()
{
	if (PostUndoRedoDelegateHandle.IsValid())
	{
		FEditorDelegates::PostUndoRedo.Remove(PostUndoRedoDelegateHandle);
		PostUndoRedoDelegateHandle.Reset();
	}
}

void SGameplayTagPicker::Construct(const FArguments& InArgs)
{
	TagContainers = InArgs._TagContainers;
	if (InArgs._PropertyHandle.IsValid())
	{
		// If we're backed by a property handle then try and get the tag containers from the property handle
		GetEditableTagContainersFromPropertyHandle(InArgs._PropertyHandle.ToSharedRef(), TagContainers);
	}

	// If we're in management mode, we don't need to have editable tag containers.
	ensure(TagContainers.Num() > 0 || InArgs._GameplayTagPickerMode == EGameplayTagPickerMode::ManagementMode);

	OnTagChanged = InArgs._OnTagChanged;
	OnRefreshTagContainers = InArgs._OnRefreshTagContainers;
	bReadOnly = InArgs._ReadOnly;
	SettingsName = InArgs._SettingsName;
	bMultiSelect = InArgs._MultiSelect;
	PropertyHandle = InArgs._PropertyHandle;
	RootFilterString = InArgs._Filter;
	GameplayTagPickerMode = InArgs._GameplayTagPickerMode;

	bDelayRefresh = false;
	MaxHeight = InArgs._MaxHeight;

	bRestrictedTags = InArgs._RestrictedTags;

	PostUndoRedoDelegateHandle = FEditorDelegates::PostUndoRedo.AddSP(this, &SGameplayTagPicker::OnPostUndoRedo);
	
	IGameplayTagsModule::OnGameplayTagTreeChanged.AddSP(this, &SGameplayTagPicker::RefreshOnNextTick);
	UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

	GetFilteredGameplayRootTags(RootFilterString, TagItems);

	if (bRestrictedTags)
	{
		// We only want to show the restricted gameplay tags
		for (int32 Idx = TagItems.Num() - 1; Idx >= 0; --Idx)
		{
			if (!TagItems[Idx]->IsRestrictedGameplayTag())
			{
				TagItems.RemoveAtSwap(Idx);
			}
		}
	}

	if (Manager.OnFilterGameplayTag.IsBound())
	{
		for (int32 Idx = TagItems.Num() - 1; Idx >= 0; --Idx)
		{
			bool DelegateShouldHide = false;
			FGameplayTagSource* Source = Manager.FindTagSource(TagItems[Idx]->GetFirstSourceName());
			Manager.OnFilterGameplayTag.Broadcast(UGameplayTagsManager::FFilterGameplayTagContext(RootFilterString, TagItems[Idx], Source, PropertyHandle), DelegateShouldHide);
			if (DelegateShouldHide)
			{
				TagItems.RemoveAtSwap(Idx);
			}
		}
	}
	
	TSharedPtr<SComboButton> SettingsCombo = SNew(SComboButton)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		.ButtonStyle(FAppStyle::Get(), "SimpleButton")
		.HasDownArrow(false)
		.ButtonContent()
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush("Icons.Settings"))
			.ColorAndOpacity(FSlateColor::UseForeground())
		];
	SettingsCombo->SetOnGetMenuContent(FOnGetContent::CreateSP(this, &SGameplayTagPicker::MakeSettingsMenu, SettingsCombo));


	TWeakPtr<SGameplayTagPicker> WeakSelf = SharedThis(this);
	
	TSharedRef<SWidget> Picker = 
		SNew(SBorder)
		.Padding(InArgs._Padding)
		.BorderImage(FStyleDefaults::GetNoBrush())
		[
			SNew(SVerticalBox)

			// Gameplay Tag Tree controls
			+SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Top)
			[
				SNew(SHorizontalBox)

				// Smaller add button for selection and hybrid modes.
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SHorizontalBox)

					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(0,0,4,0))
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
						.ToolTipText_Lambda([WeakSelf]() -> FText
						{
							const TSharedPtr<SGameplayTagPicker> Self = WeakSelf.Pin();
							if (Self.IsValid() && Self->bNewTagWidgetVisible)
							{
								return LOCTEXT("CloseSection", "Close Section");
							}
							return LOCTEXT("AddNewGameplayTag", "Add New Gameplay Tag");
						})
						.OnClicked_Lambda([WeakSelf]()
						{
							if (const TSharedPtr<SGameplayTagPicker> Self = WeakSelf.Pin())
							{
								if (!Self->bNewTagWidgetVisible)
								{
									// If we have a selected item, by default add child, else new root tag. 
									TArray<TSharedPtr<FGameplayTagNode>> Selection = Self->TagTreeWidget->GetSelectedItems();
									if (Selection.Num() > 0 && Selection[0].IsValid())
									{
										Self->ShowInlineAddTagWidget(EGameplayTagAdd::Child, Selection[0]);
									}
									else
									{
										Self->ShowInlineAddTagWidget(EGameplayTagAdd::Root);
									}
								}
								else
								{
									Self->bNewTagWidgetVisible = false;
								}
							}
							return FReply::Handled();
						})
						[
							SNew(SImage)
                        	.Image(FAppStyle::GetBrush("Icons.Plus"))
                        	.ColorAndOpacity(FStyleColors::AccentGreen)
						]
					]
				]

				// Search
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.FillWidth(1.f)
				.Padding(0,1,5,1)
				[
					SAssignNew(SearchTagBox, SSearchBox)
					.HintText(LOCTEXT("GameplayTagPicker_SearchBoxHint", "Search tags matching ALL keywords"))
					.OnTextChanged(this, &SGameplayTagPicker::OnFilterTextChanged)
				]

				// View settings
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SettingsCombo.ToSharedRef()
				]
			]

			// Inline add new tag window for selection and hybrid modes.
			+SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Top)
			.Padding(FMargin(4,2))
			[
				SAssignNew(AddNewTagWidget, SAddNewGameplayTagWidget )
				.Padding(FMargin(0, 2))
				.AddButtonPadding(FMargin(0, 4, 0, 0))
				.RestrictedTags(bRestrictedTags)
				.Visibility_Lambda([WeakSelf]()
				{
					const TSharedPtr<SGameplayTagPicker> Self = WeakSelf.Pin();
					return (Self.IsValid() && Self->bNewTagWidgetVisible) ? EVisibility::Visible : EVisibility::Collapsed;
				})
				.OnGameplayTagAdded_Lambda([WeakSelf](const FString& TagName, const FString& TagComment, const FName& TagSource)
				{
				   if (const TSharedPtr<SGameplayTagPicker> Self = WeakSelf.Pin())
				   {
					   Self->OnGameplayTagAdded(TagName, TagComment, TagSource);
				   }
				})
			]

			// Gameplay Tags tree
			+SVerticalBox::Slot()
			.MaxHeight(MaxHeight)
			.FillHeight(1)
			[
				SAssignNew(TagTreeContainerWidget, SBorder)
				.BorderImage(FStyleDefaults::GetNoBrush())
				[
					SAssignNew(TagTreeWidget, STreeView<TSharedPtr<FGameplayTagNode>>)
					.TreeItemsSource(&TagItems)
					.OnGenerateRow(this, &SGameplayTagPicker::OnGenerateRow)
					.OnGetChildren(this, &SGameplayTagPicker::OnGetChildren)
					.OnExpansionChanged(this, &SGameplayTagPicker::OnExpansionChanged)
					.SelectionMode(ESelectionMode::Single)
					.OnContextMenuOpening(this, &SGameplayTagPicker::OnTreeContextMenuOpening)
					.OnSelectionChanged(this, &SGameplayTagPicker::OnTreeSelectionChanged)
					.OnKeyDownHandler(this, &SGameplayTagPicker::OnTreeKeyDown)
				]
			]
		];

	if (InArgs._ShowMenuItems)
	{
		FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection*/false, nullptr);

		if (InArgs._MultiSelect)
		{
			MenuBuilder.BeginSection(FName(), LOCTEXT("SectionGameplayTagContainer", "GameplayTag Container"));
		}
		else
		{
			MenuBuilder.BeginSection(FName(), LOCTEXT("SectionGameplayTag", "GameplayTag"));
		}
		
		MenuBuilder.AddMenuEntry(
			LOCTEXT("GameplayTagPicker_ClearSelection", "Clear Selection"), FText::GetEmpty(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.X"),
			FUIAction(FExecuteAction::CreateRaw(this, &SGameplayTagPicker::OnClearAllClicked, TSharedPtr<SComboButton>()))
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("GameplayTagPicker_ManageTags", "Manage Gameplay Tags..."), FText::GetEmpty(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Settings"),
			FUIAction(FExecuteAction::CreateRaw(this, &SGameplayTagPicker::OnManageTagsClicked, TSharedPtr<FGameplayTagNode>(), TSharedPtr<SComboButton>()))
		);

		MenuBuilder.AddSeparator();

		TSharedRef<SWidget> MenuContent =
			SNew(SBox)
			.WidthOverride(300.0f)
			.HeightOverride(MaxHeight)
			[
				Picker
			];
		MenuBuilder.AddWidget(MenuContent, FText::GetEmpty(), true);

		MenuBuilder.EndSection();
		
		ChildSlot
		[
			MenuBuilder.MakeWidget()
		];
	}
	else
	{
		ChildSlot
		[
			Picker
		];
	}
	
	// Force the entire tree collapsed to start
	SetTagTreeItemExpansion(/*bExpand*/false, /*bPersistExpansion*/false);

	LoadSettings();

	VerifyAssetTagValidity();
}

TSharedPtr<SWidget> SGameplayTagPicker::OnTreeContextMenuOpening()
{
	TArray<TSharedPtr<FGameplayTagNode>> Selection = TagTreeWidget->GetSelectedItems();
	const TSharedPtr<FGameplayTagNode> SelectedTagNode = Selection.IsEmpty() ? nullptr : Selection[0];
	return MakeTagActionsMenu(SelectedTagNode, TSharedPtr<SComboButton>(), /*bInShouldCloseWindowAfterMenuSelection*/true);
}

void SGameplayTagPicker::OnTreeSelectionChanged(TSharedPtr<FGameplayTagNode> SelectedItem, ESelectInfo::Type SelectInfo)
{
	if (GameplayTagPickerMode == EGameplayTagPickerMode::SelectionMode
		&& SelectInfo == ESelectInfo::OnMouseClick)
	{
		// In selection mode we do not allow to select lines as they have not meaning,
		// but the highlight helps navigating the list.   
		if (!bInSelectionChanged)
		{
			TGuardValue<bool> PersistExpansionChangeGuard(bInSelectionChanged, true);
			TagTreeWidget->ClearSelection();

			// Toggle selection
			const ECheckBoxState State = IsTagChecked(SelectedItem);
			if (State == ECheckBoxState::Unchecked)
			{
				OnTagChecked(SelectedItem);
			}
			else if (State == ECheckBoxState::Checked)
			{
				OnTagUnchecked(SelectedItem);
			}
		}
	}
}

FReply SGameplayTagPicker::OnTreeKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
{
	if (FSlateApplication::Get().GetNavigationActionFromKey(InKeyEvent) == EUINavigationAction::Accept)
	{
		TArray<TSharedPtr<FGameplayTagNode>> Selection = TagTreeWidget->GetSelectedItems();
		TSharedPtr<FGameplayTagNode> SelectedItem;
		if (!Selection.IsEmpty())
		{
			SelectedItem = Selection[0];
		}

		if (SelectedItem.IsValid())
		{
			TGuardValue<bool> PersistExpansionChangeGuard(bInSelectionChanged, true);

			// Toggle selection
			const ECheckBoxState State = IsTagChecked(SelectedItem);
			if (State == ECheckBoxState::Unchecked)
			{
				OnTagChecked(SelectedItem);
			}
			else if (State == ECheckBoxState::Checked)
			{
				OnTagUnchecked(SelectedItem);
			}
			
			return FReply::Handled();
		}
	}

	return SCompoundWidget::OnKeyDown(InGeometry, InKeyEvent);
}

TSharedRef<SWidget> SGameplayTagPicker::MakeSettingsMenu(TSharedPtr<SComboButton> OwnerCombo)
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection*/false, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("GameplayTagPicker_ExpandAll", "Expand All"), FText::GetEmpty(), FSlateIcon(),
		FUIAction(FExecuteAction::CreateRaw(this, &SGameplayTagPicker::OnExpandAllClicked, OwnerCombo))
	);
	
	MenuBuilder.AddMenuEntry(
		LOCTEXT("GameplayTagPicker_CollapseAll", "Collapse All"), FText::GetEmpty(), FSlateIcon(),
		FUIAction(FExecuteAction::CreateRaw(this, &SGameplayTagPicker::OnCollapseAllClicked, OwnerCombo))
	);
	
	return MenuBuilder.MakeWidget();
}

void SGameplayTagPicker::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (PropertyHandle.IsValid())
	{
		// If we're backed by a property handle then try and refresh the tag containers, 
		// as they may have changed under us (eg, from object re-instancing)
		GetEditableTagContainersFromPropertyHandle(PropertyHandle.ToSharedRef(), TagContainers);
	}

	if (bDelayRefresh)
	{
		RefreshTags();
		bDelayRefresh = false;
	}

	if (RequestedScrollToTag.IsValid())
	{
		// Scroll specified item into view.
		UGameplayTagsManager& Manager = UGameplayTagsManager::Get();
		TSharedPtr<FGameplayTagNode> Node = Manager.FindTagNode(RequestedScrollToTag);

		if (Node.IsValid())
		{
			// Expand all the parent nodes to make sure the target node is visible.
			TSharedPtr<FGameplayTagNode> ParentNode = Node.IsValid() ? Node->ParentNode : nullptr;
			while (ParentNode.IsValid())
			{
				TagTreeWidget->SetItemExpansion(ParentNode, /*bExpand*/true);
				ParentNode = ParentNode->ParentNode;
			}

			TagTreeWidget->ClearSelection();
			TagTreeWidget->SetItemSelection(Node, true);
			TagTreeWidget->RequestScrollIntoView(Node);
		}
		
		RequestedScrollToTag = FGameplayTag();
	}
}

void SGameplayTagPicker::OnFilterTextChanged(const FText& InFilterText)
{
	InFilterText.ToString().ParseIntoArrayWS(FilterStringTokens);
	FilterTagTree();
}

void SGameplayTagPicker::FilterTagTree()
{
	if (FilterStringTokens.IsEmpty())
	{
		TagTreeWidget->SetTreeItemsSource(&TagItems);

		for (int32 iItem = 0; iItem < TagItems.Num(); ++iItem)
		{
			SetDefaultTagNodeItemExpansion(TagItems[iItem]);
		}
	}
	else
	{
		FilteredTagItems.Empty();

		for (int32 iItem = 0; iItem < TagItems.Num(); ++iItem)
		{
			if (FilterChildrenCheck(TagItems[iItem]))
			{
				FilteredTagItems.Add(TagItems[iItem]);
				SetTagNodeItemExpansion(TagItems[iItem], true);
			}
			else
			{
				SetTagNodeItemExpansion(TagItems[iItem], false);
			}
		}

		TagTreeWidget->SetTreeItemsSource(&FilteredTagItems);
	}

	TagTreeWidget->RequestTreeRefresh();
}

bool SGameplayTagPicker::FilterChildrenCheckRecursive(TSharedPtr<FGameplayTagNode>& InItem) const
{
	for (TSharedPtr<FGameplayTagNode>& Child : InItem->GetChildTagNodes())
	{
		if (FilterChildrenCheck(Child))
		{
			return true;
		}
	}
	return false;
}

bool SGameplayTagPicker::FilterChildrenCheck(TSharedPtr<FGameplayTagNode>& InItem) const
{
	if (!InItem.IsValid())
	{
		return false;
	}

	if (bRestrictedTags && !InItem->IsRestrictedGameplayTag())
	{
		return false;
	}

	UGameplayTagsManager& Manager = UGameplayTagsManager::Get();
	bool bDelegateShouldHide = false;
	Manager.OnFilterGameplayTagChildren.Broadcast(RootFilterString, InItem, bDelegateShouldHide);
	if (!bDelegateShouldHide && Manager.OnFilterGameplayTag.IsBound())
	{
		FGameplayTagSource* Source = Manager.FindTagSource(InItem->GetFirstSourceName());
		Manager.OnFilterGameplayTag.Broadcast(UGameplayTagsManager::FFilterGameplayTagContext(RootFilterString, InItem, Source, PropertyHandle), bDelegateShouldHide);
	}
	if (bDelegateShouldHide)
	{
		// The delegate wants to hide, see if any children need to show
		return FilterChildrenCheckRecursive(InItem);
	}

	if (!FilterStringTokens.IsEmpty())
	{
		const FString CompleteTagString = InItem->GetCompleteTagString();
		for (const FString& Token : FilterStringTokens)
		{
			if (!CompleteTagString.Contains(Token))
			{
				return FilterChildrenCheckRecursive(InItem);
			}
		}
	}

	// All tokens matched (or there are no search tokens)
	return true;
}

FText SGameplayTagPicker::GetHighlightText() const
{
	return FilterStringTokens.IsEmpty() ? FText::GetEmpty() : FText::FromString(FilterStringTokens.Last());
}

TSharedRef<ITableRow> SGameplayTagPicker::OnGenerateRow(TSharedPtr<FGameplayTagNode> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	FText TooltipText;
	FString TagSource;
	bool bIsExplicitTag = true;
	if (InItem.IsValid())
	{
		UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

		FName TagName = InItem.Get()->GetCompleteTagName();
		TSharedPtr<FGameplayTagNode> Node = Manager.FindTagNode(TagName);

		FString TooltipString = TagName.ToString();

		if (Node.IsValid())
		{
			constexpr int32 MaxSourcesToDisplay = 3; // How many sources to display before showing ellipsis (tool tip will have all sources). 

			FString AllSources;
			for (TConstEnumerateRef<FName> Source : EnumerateRange(Node->GetAllSourceNames()))
			{
				if (AllSources.Len() > 0)
				{
					AllSources += TEXT(", ");
				}
				AllSources += Source->ToString();
				
				if (Source.GetIndex() < MaxSourcesToDisplay)
				{
					if (TagSource.Len() > 0)
					{
						TagSource += TEXT(", ");
					}
					TagSource += Source->ToString();
				}
			}
			
			if (Node->GetAllSourceNames().Num() > MaxSourcesToDisplay)
			{
				TagSource += FString::Printf(TEXT(", ... (%d)"), Node->GetAllSourceNames().Num() - MaxSourcesToDisplay);
			}

			bIsExplicitTag = Node->bIsExplicitTag;

			TooltipString.Appendf(TEXT("\n(%s%s)"), bIsExplicitTag ? TEXT("") : TEXT("Implicit "), *AllSources);

			// tag comments
			if (!Node->DevComment.IsEmpty())
			{
				TooltipString.Appendf(TEXT("\n\n%s"), *Node->DevComment);
			}

			// info related to conflicts
			if (Node->bDescendantHasConflict)
			{
				TooltipString.Append(TEXT("\n\nA tag that descends from this tag has a source conflict."));
			}

			if (Node->bAncestorHasConflict)
			{
				TooltipString.Append(TEXT("\n\nThis tag is descended from a tag that has a conflict. No operations can be performed on this tag until the conflict is resolved."));
			}

			if (Node->bNodeHasConflict)
			{
				TooltipString.Append(TEXT("\n\nThis tag comes from multiple sources. Tags may only have one source."));
			}
		}

		TooltipText = FText::FromString(MoveTemp(TooltipString));
	}

	TSharedPtr<SComboButton> ActionsCombo = SNew(SComboButton)
		.ToolTipText(LOCTEXT("MoreActions", "More Actions..."))
		.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
		.ContentPadding(0)
		.ForegroundColor(FSlateColor::UseForeground())
		.HasDownArrow(true)
		.CollapseMenuOnParentFocus(true);

	// Craete context menu with bInShouldCloseWindowAfterMenuSelection = false, or else the actions menu action will not work due the popup-menu handling order.
	ActionsCombo->SetOnGetMenuContent(FOnGetContent::CreateSP(this, &SGameplayTagPicker::MakeTagActionsMenu, InItem, ActionsCombo, /*bInShouldCloseWindowAfterMenuSelection*/false));
	
	if (GameplayTagPickerMode == EGameplayTagPickerMode::SelectionMode
		|| GameplayTagPickerMode == EGameplayTagPickerMode::HybridMode)
	{
		return SNew(STableRow<TSharedPtr<FGameplayTagNode>>, OwnerTable)
		.Style(FAppStyle::Get(), "GameplayTagTreeView")
		.ToolTipText(TooltipText)
		[
			SNew(SHorizontalBox)

			// Tag Selection (selection mode only)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(SCheckBox)
				.OnCheckStateChanged(this, &SGameplayTagPicker::OnTagCheckStatusChanged, InItem)
				.IsChecked(this, &SGameplayTagPicker::IsTagChecked, InItem)
				.IsEnabled(this, &SGameplayTagPicker::CanSelectTags)
				.CheckBoxContentUsesAutoWidth(false)
				[
					SNew(STextBlock)
					.HighlightText(this, &SGameplayTagPicker::GetHighlightText)
					.Text(FText::FromName(InItem->GetSimpleTagName()))
				]
			]

			// Allows non-restricted children checkbox
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				SNew(SCheckBox)
				.ToolTipText(LOCTEXT("AllowsChildren", "Does this restricted tag allow non-restricted children"))
				.OnCheckStateChanged(this, &SGameplayTagPicker::OnAllowChildrenTagCheckStatusChanged, InItem)
				.IsChecked(this, &SGameplayTagPicker::IsAllowChildrenTagChecked, InItem)
				.Visibility(this, &SGameplayTagPicker::DetermineAllowChildrenVisible, InItem)
			]

			// More Actions Menu
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				ActionsCombo.ToSharedRef()
			]
		];
	}
	else
	{
		return SNew(STableRow<TSharedPtr<FGameplayTagNode>>, OwnerTable)
		.Style(FAppStyle::Get(), "GameplayTagTreeView")
		[
			SNew(SHorizontalBox)

			// Normal Tag Display (management mode only)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.ToolTip(FSlateApplication::Get().MakeToolTip(TooltipText))
				.Text(FText::FromName(InItem->GetSimpleTagName()))
				.ColorAndOpacity(this, &SGameplayTagPicker::GetTagTextColour, InItem)
				.HighlightText(this, &SGameplayTagPicker::GetHighlightText)
			]

			// Source
			+SHorizontalBox::Slot()
			.FillWidth(1)
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(FMargin(16,0, 4, 0))
			[
				SNew(STextBlock)
				.Clipping(EWidgetClipping::OnDemand)
				.ToolTip(FSlateApplication::Get().MakeToolTip(TooltipText))
				.Text(FText::FromString(TagSource) )
				.ColorAndOpacity(bIsExplicitTag ? FLinearColor(1,1,1,0.5f) : FLinearColor(1,1,1,0.25f))
			]

			// Allows non-restricted children checkbox
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				SNew(SCheckBox)
				.ToolTipText(LOCTEXT("AllowsChildren", "Does this restricted tag allow non-restricted children"))
				.OnCheckStateChanged(this, &SGameplayTagPicker::OnAllowChildrenTagCheckStatusChanged, InItem)
				.IsChecked(this, &SGameplayTagPicker::IsAllowChildrenTagChecked, InItem)
				.Visibility(this, &SGameplayTagPicker::DetermineAllowChildrenVisible, InItem)
			]

			// More Actions Menu
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				ActionsCombo.ToSharedRef()
			]
		];
	}
}

void SGameplayTagPicker::OnGetChildren(TSharedPtr<FGameplayTagNode> InItem, TArray<TSharedPtr<FGameplayTagNode>>& OutChildren)
{
	TArray<TSharedPtr<FGameplayTagNode>> FilteredChildren;
	TArray<TSharedPtr<FGameplayTagNode>> Children = InItem->GetChildTagNodes();

	for (int32 iChild = 0; iChild < Children.Num(); ++iChild)
	{
		if (FilterChildrenCheck(Children[iChild]))
		{
			FilteredChildren.Add(Children[iChild]);
		}
	}
	OutChildren += FilteredChildren;
}

void SGameplayTagPicker::OnTagCheckStatusChanged(ECheckBoxState NewCheckState, TSharedPtr<FGameplayTagNode> NodeChanged)
{
	if (NewCheckState == ECheckBoxState::Checked)
	{
		OnTagChecked(NodeChanged);
	}
	else if (NewCheckState == ECheckBoxState::Unchecked)
	{
		OnTagUnchecked(NodeChanged);
	}
}

void SGameplayTagPicker::OnTagChecked(TSharedPtr<FGameplayTagNode> NodeChecked)
{
	FScopedTransaction Transaction(LOCTEXT("GameplayTagPicker_SelectTags", "Select Gameplay Tags"));

	UGameplayTagsManager& TagsManager = UGameplayTagsManager::Get();

	for (FGameplayTagContainer& Container : TagContainers)
	{
		TSharedPtr<FGameplayTagNode> CurNode(NodeChecked);

		bool bRemoveParents = false;

		while (CurNode.IsValid())
		{
			FGameplayTag GameplayTag = CurNode->GetCompleteTag();

			if (bRemoveParents == false)
			{
				bRemoveParents = true;
				if (bMultiSelect == false)
				{
					Container.Reset();
				}
				Container.AddTag(GameplayTag);
			}
			else
			{
				Container.RemoveTag(GameplayTag);
			}

			CurNode = CurNode->GetParentTagNode();
		}
	}

	OnContainersChanged();
}

void SGameplayTagPicker::OnTagUnchecked(TSharedPtr<FGameplayTagNode> NodeUnchecked)
{
	FScopedTransaction Transaction(LOCTEXT("GameplayTagPicker_RemoveTags", "Remove Gameplay Tags"));
	if (NodeUnchecked.IsValid())
	{
		UGameplayTagsManager& TagsManager = UGameplayTagsManager::Get();

		for (FGameplayTagContainer& Container : TagContainers)
		{
			FGameplayTag GameplayTag = NodeUnchecked->GetCompleteTag();

			Container.RemoveTag(GameplayTag);

			TSharedPtr<FGameplayTagNode> ParentNode = NodeUnchecked->GetParentTagNode();
			if (ParentNode.IsValid())
			{
				// Check if there are other siblings before adding parent
				bool bOtherSiblings = false;
				for (auto It = ParentNode->GetChildTagNodes().CreateConstIterator(); It; ++It)
				{
					GameplayTag = It->Get()->GetCompleteTag();
					if (Container.HasTagExact(GameplayTag))
					{
						bOtherSiblings = true;
						break;
					}
				}
				// Add Parent
				if (!bOtherSiblings)
				{
					GameplayTag = ParentNode->GetCompleteTag();
					Container.AddTag(GameplayTag);
				}
			}

			// Uncheck Children
			for (const auto& ChildNode : NodeUnchecked->GetChildTagNodes())
			{
				UncheckChildren(ChildNode, Container);
			}
		}
		
		OnContainersChanged();
	}
}

void SGameplayTagPicker::UncheckChildren(TSharedPtr<FGameplayTagNode> NodeUnchecked, FGameplayTagContainer& EditableContainer)
{
	UGameplayTagsManager& TagsManager = UGameplayTagsManager::Get();

	FGameplayTag GameplayTag = NodeUnchecked->GetCompleteTag();
	EditableContainer.RemoveTag(GameplayTag);

	// Uncheck Children
	for (const auto& ChildNode : NodeUnchecked->GetChildTagNodes())
	{
		UncheckChildren(ChildNode, EditableContainer);
	}
}

ECheckBoxState SGameplayTagPicker::IsTagChecked(TSharedPtr<FGameplayTagNode> Node) const
{
	int32 NumValidAssets = 0;
	int32 NumAssetsTagIsAppliedTo = 0;

	if (Node.IsValid())
	{
		UGameplayTagsManager& TagsManager = UGameplayTagsManager::Get();

		for (const FGameplayTagContainer& Container : TagContainers)
		{
			NumValidAssets++;
			const FGameplayTag GameplayTag = Node->GetCompleteTag();
			if (GameplayTag.IsValid())
			{
				if (Container.HasTag(GameplayTag))
				{
					++NumAssetsTagIsAppliedTo;
				}
			}
		}
	}

	if (NumAssetsTagIsAppliedTo == 0)
	{
		return ECheckBoxState::Unchecked;
	}
	else if (NumAssetsTagIsAppliedTo == NumValidAssets)
	{
		return ECheckBoxState::Checked;
	}
	else
	{
		return ECheckBoxState::Undetermined;
	}
}

bool SGameplayTagPicker::IsExactTagInCollection(TSharedPtr<FGameplayTagNode> Node) const
{
	if (Node.IsValid())
	{
		for (const FGameplayTagContainer& Container : TagContainers)
		{
			FGameplayTag GameplayTag = Node->GetCompleteTag();
			if (GameplayTag.IsValid())
			{
				if (Container.HasTagExact(GameplayTag))
				{
					return true;
				}
			}
		}
	}

	return false;
}

void SGameplayTagPicker::OnAllowChildrenTagCheckStatusChanged(ECheckBoxState NewCheckState, TSharedPtr<FGameplayTagNode> NodeChanged)
{
	IGameplayTagsEditorModule& TagsEditor = IGameplayTagsEditorModule::Get();

	if (TagsEditor.UpdateTagInINI(NodeChanged->GetCompleteTagString(), NodeChanged->DevComment, NodeChanged->bIsRestrictedTag, NewCheckState == ECheckBoxState::Checked))
	{
		if (NewCheckState == ECheckBoxState::Checked)
		{
			NodeChanged->bAllowNonRestrictedChildren = true;
		}
		else if (NewCheckState == ECheckBoxState::Unchecked)
		{
			NodeChanged->bAllowNonRestrictedChildren = false;
		}
	}
}

ECheckBoxState SGameplayTagPicker::IsAllowChildrenTagChecked(TSharedPtr<FGameplayTagNode> Node) const
{
	if (Node->GetAllowNonRestrictedChildren())
	{
		return ECheckBoxState::Checked;
	}

	return ECheckBoxState::Unchecked;
}

EVisibility SGameplayTagPicker::DetermineAllowChildrenVisible(TSharedPtr<FGameplayTagNode> Node) const
{
	// We do not allow you to modify nodes that have a conflict or inherit from a node with a conflict
	if (Node->bNodeHasConflict || Node->bAncestorHasConflict)
	{
		return EVisibility::Hidden;
	}

	if (bRestrictedTags)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

void SGameplayTagPicker::OnManageTagsClicked(TSharedPtr<FGameplayTagNode> Node, TSharedPtr<SComboButton> OwnerCombo)
{
	FGameplayTagManagerWindowArgs Args;
	Args.bRestrictedTags = bRestrictedTags;
	Args.Filter = RootFilterString;
	if (Node.IsValid())
	{
		Args.HighlightedTag = Node->GetCompleteTag();
	}
	
	UE::GameplayTags::Editor::OpenGameplayTagManager(Args);

	if (OwnerCombo.IsValid())
	{
		OwnerCombo->SetIsOpen(false);
	}
}

void SGameplayTagPicker::OnClearAllClicked(TSharedPtr<SComboButton> OwnerCombo)
{
	FScopedTransaction Transaction(LOCTEXT("GameplayTagPicker_RemoveAllTags", "Remove All Gameplay Tags") );

	for (FGameplayTagContainer& Container : TagContainers)
	{
		Container.Reset();
	}

	OnContainersChanged();

	if (OwnerCombo.IsValid())
	{
		OwnerCombo->SetIsOpen(false);
	}
}

FSlateColor SGameplayTagPicker::GetTagTextColour(TSharedPtr<FGameplayTagNode> Node) const
{
	static const FLinearColor DefaultTextColour = FLinearColor::White;
	static const FLinearColor DescendantConflictTextColour = FLinearColor(1.f, 0.65f, 0.f); // orange
	static const FLinearColor NodeConflictTextColour = FLinearColor::Red;
	static const FLinearColor AncestorConflictTextColour = FLinearColor(1.f, 1.f, 1.f, 0.5f);

	if (Node->bNodeHasConflict)
	{
		return NodeConflictTextColour;
	}

	if (Node->bDescendantHasConflict)
	{
		return DescendantConflictTextColour;
	}

	if (Node->bAncestorHasConflict)
	{
		return AncestorConflictTextColour;
	}

	return DefaultTextColour;
}

void SGameplayTagPicker::OnExpandAllClicked(TSharedPtr<SComboButton> OwnerCombo)
{
	SetTagTreeItemExpansion(/*bExpand*/true, /*bPersistExpansion*/true);
	if (OwnerCombo.IsValid())
	{
		OwnerCombo->SetIsOpen(false);
	}
}

void SGameplayTagPicker::OnCollapseAllClicked(TSharedPtr<SComboButton> OwnerCombo)
{
	SetTagTreeItemExpansion(/*bExpand*/false, /*bPersistExpansion*/true);
	if (OwnerCombo.IsValid())
	{
		OwnerCombo->SetIsOpen(false);
	}
}

void SGameplayTagPicker::ShowInlineAddTagWidget(const EGameplayTagAdd Mode, TSharedPtr<FGameplayTagNode> InTagNode)
{
	UGameplayTagsManager& Manager = UGameplayTagsManager::Get();
	
	FString TagName;
	FName TagFName;
	FString TagComment;
	FName TagSource;
	bool bTagIsExplicit;
	bool bTagIsRestricted;
	bool bTagAllowsNonRestrictedChildren;

	if (InTagNode.IsValid())
	{
		TagName = InTagNode->GetCompleteTagString();
		TagFName = InTagNode->GetCompleteTagName();
	}

	Manager.GetTagEditorData(TagFName, TagComment, TagSource, bTagIsExplicit, bTagIsRestricted, bTagAllowsNonRestrictedChildren);

	if (AddNewTagWidget.IsValid())
	{
		if (Mode == EGameplayTagAdd::Child || Mode == EGameplayTagAdd::Root)
		{
			AddNewTagWidget->AddSubtagFromParent(TagName, TagSource);
		}
		else if (Mode == EGameplayTagAdd::Duplicate)
		{
			AddNewTagWidget->AddDuplicate(TagName, TagSource);
		}
		bNewTagWidgetVisible = true;
	}
}

bool SGameplayTagPicker::IsNodeSingleRoot(const TSharedPtr<FGameplayTagNode>& InNode) const
{
	return TagItems.Num() == 1 && TagItems[0] == InNode;
}

TSharedRef<SWidget> SGameplayTagPicker::MakeTagActionsMenu(TSharedPtr<FGameplayTagNode> InTagNode, TSharedPtr<SComboButton> ActionsCombo, bool bInShouldCloseWindowAfterMenuSelection)
{
	if (!InTagNode.IsValid())
	{
		return SNullWidget::NullWidget;
	}
	
	bool bShowManagement = ((GameplayTagPickerMode == EGameplayTagPickerMode::ManagementMode || GameplayTagPickerMode == EGameplayTagPickerMode::HybridMode) && !bReadOnly);
	UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

	if (!Manager.ShouldImportTagsFromINI())
	{
		bShowManagement = false;
	}

	// You can't modify restricted tags in the normal tag menus
	if (!bRestrictedTags && InTagNode->IsRestrictedGameplayTag())
	{
		bShowManagement = false;
	}

	// Do not close menu after selection. The close deletes this widget before action is executed leading to no action being performed.
	// Occurs when SGameplayTagPicker is being used as a menu item itself (Details panel of blueprint editor for example).
	FMenuBuilder MenuBuilder(bInShouldCloseWindowAfterMenuSelection, nullptr);

	// Add child tag
	MenuBuilder.AddMenuEntry(LOCTEXT("GameplayTagPicker_AddSubTag", "Add Sub Tag"),
		LOCTEXT("GameplayTagPicker_AddSubTagTagTooltip", "Add sub tag under selected tag."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Plus"),
		FUIAction(FExecuteAction::CreateSP(this, &SGameplayTagPicker::OnAddSubTag, InTagNode, ActionsCombo), FCanExecuteAction::CreateSP(this, &SGameplayTagPicker::CanAddNewSubTag, InTagNode)));

	// Duplicate
	MenuBuilder.AddMenuEntry(LOCTEXT("GameplayTagPicker_DuplicateTag", "Duplicate Tag"),
		LOCTEXT("GameplayTagPicker_DuplicateTagTooltip", "Duplicate selected tag to create a new tag."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Duplicate"),
		FUIAction(FExecuteAction::CreateSP(this, &SGameplayTagPicker::OnDuplicateTag, InTagNode, ActionsCombo), FCanExecuteAction::CreateSP(this, &SGameplayTagPicker::CanAddNewSubTag, InTagNode)));

	MenuBuilder.AddSeparator();

	if (bShowManagement)
	{
		// Rename
		MenuBuilder.AddMenuEntry(LOCTEXT("GameplayTagPicker_RenameTag", "Rename Tag"),
			LOCTEXT("GameplayTagPicker_RenameTagTooltip", "Rename this tag"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Edit"),
			FUIAction(FExecuteAction::CreateSP(this, &SGameplayTagPicker::OnRenameTag, InTagNode, ActionsCombo), FCanExecuteAction::CreateSP(this, &SGameplayTagPicker::CanModifyTag, InTagNode)));

		// Delete
		MenuBuilder.AddMenuEntry(LOCTEXT("GameplayTagPicker_DeleteTag", "Delete Tag"),
			LOCTEXT("GameplayTagPicker_DeleteTagTooltip", "Delete this tag"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Delete"),
			FUIAction(FExecuteAction::CreateSP(this, &SGameplayTagPicker::OnDeleteTag, InTagNode, ActionsCombo), FCanExecuteAction::CreateSP(this, &SGameplayTagPicker::CanModifyTag, InTagNode)));

		MenuBuilder.AddSeparator();
	}

	// Only include these menu items if we have tag containers to modify
	if (bMultiSelect)
	{
		// Either Selector or Unselect Exact Tag depending on if we have the exact tag or not
		if (IsExactTagInCollection(InTagNode))
		{
			MenuBuilder.AddMenuEntry(LOCTEXT("GameplayTagPicker_UnselectTag", "Unselect Exact Tag"),
				LOCTEXT("GameplayTagPicker_RemoveTagTooltip", "Unselect this exact tag, Parent and Child Tags will not be effected."),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &SGameplayTagPicker::OnUnselectExactTag, InTagNode, ActionsCombo)));
		}
		else
		{
			MenuBuilder.AddMenuEntry(LOCTEXT("GameplayTagPicker_SelectTag", "Select Exact Tag"),
				LOCTEXT("GameplayTagPicker_AddTagTooltip", "Select this exact tag, Parent and Child Child Tags will not be effected."),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &SGameplayTagPicker::OnSelectExactTag, InTagNode, ActionsCombo)));
		}

		MenuBuilder.AddSeparator();
	}

	if (!bShowManagement)
	{
		// Open tag in manager
		MenuBuilder.AddMenuEntry(LOCTEXT("GameplayTagPicker_OpenTagInManager", "Open Tag in Manager..."),
			LOCTEXT("GameplayTagPicker_OpenTagInManagerTooltip", "Opens the Gameplay Tag manage and hilights the selected tag."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Settings"),
			FUIAction(FExecuteAction::CreateRaw(this, &SGameplayTagPicker::OnManageTagsClicked, InTagNode, TSharedPtr<SComboButton>())));
	}

	// Search for References
	if (FEditorDelegates::OnOpenReferenceViewer.IsBound())
	{
		MenuBuilder.AddMenuEntry(LOCTEXT("GameplayTagPicker_SearchForReferences", "Search For References"),
		FText::Format(LOCTEXT("GameplayTagPicker_SearchForReferencesTooltip", "Find references to the tag {0}"), FText::AsCultureInvariant(InTagNode->GetCompleteTagString())),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Search"),
			FUIAction(FExecuteAction::CreateSP(this, &SGameplayTagPicker::OnSearchForReferences, InTagNode, ActionsCombo)));
	}

	// Search for References of all children
	if (FEditorDelegates::OnOpenReferenceViewer.IsBound())
	{
		MenuBuilder.AddMenuEntry(LOCTEXT("GameplayTagPicker_SearchForReferencesAsChildren", "Search For References (all children)"),
			FText::Format(LOCTEXT("GameplayTagPicker_SearchForReferencesAsChildrenTooltip", "Find references to the tag {0} and all its children"), FText::AsCultureInvariant(InTagNode->GetCompleteTagString())),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Search"),
			FUIAction(FExecuteAction::CreateSP(this, &SGameplayTagPicker::OnSearchForReferencesAsChildren, InTagNode, ActionsCombo)));
	}

	// Copy Name to Clipboard
	MenuBuilder.AddMenuEntry(LOCTEXT("GameplayTagPicker_CopyNameToClipboard", "Copy Name to Clipboard"),
	FText::Format(LOCTEXT("GameplayTagPicker_CopyNameToClipboardTooltip", "Copy tag {0} to clipboard"), FText::AsCultureInvariant(InTagNode->GetCompleteTagString())),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Copy"),
		FUIAction(FExecuteAction::CreateSP(this, &SGameplayTagPicker::OnCopyTagNameToClipboard, InTagNode, ActionsCombo)));

	return MenuBuilder.MakeWidget();
}

bool SGameplayTagPicker::CanModifyTag(TSharedPtr<FGameplayTagNode> Node) const
{
	if (Node.IsValid())
	{
		// we can only modify tags if they came from an ini file
		if (Node->GetFirstSourceName().ToString().EndsWith(TEXT(".ini")))
		{
			return true;
		}
	}
	return false;
}

void SGameplayTagPicker::OnAddSubTag(TSharedPtr<FGameplayTagNode> InTagNode, TSharedPtr<SComboButton> OwnerCombo)
{
	if (InTagNode.IsValid())
	{
		ShowInlineAddTagWidget(EGameplayTagAdd::Child, InTagNode);
	}

	if (OwnerCombo.IsValid())
	{
		OwnerCombo->SetIsOpen(false);
	}
}

void SGameplayTagPicker::OnDuplicateTag(TSharedPtr<FGameplayTagNode> InTagNode, TSharedPtr<SComboButton> OwnerCombo)
{
	if (InTagNode.IsValid())
	{
		ShowInlineAddTagWidget(EGameplayTagAdd::Duplicate, InTagNode);
	}
	
	if (OwnerCombo.IsValid())
	{
		OwnerCombo->SetIsOpen(false);
	}
}

void SGameplayTagPicker::OnRenameTag(TSharedPtr<FGameplayTagNode> InTagNode, TSharedPtr<SComboButton> OwnerCombo)
{
	if (InTagNode.IsValid())
	{
		OpenRenameGameplayTagDialog(InTagNode);
	}
	
	if (OwnerCombo.IsValid())
	{
		OwnerCombo->SetIsOpen(false);
	}
}

void SGameplayTagPicker::OnDeleteTag(TSharedPtr<FGameplayTagNode> InTagNode, TSharedPtr<SComboButton> OwnerCombo)
{
	if (InTagNode.IsValid())
	{
		IGameplayTagsEditorModule& TagsEditor = IGameplayTagsEditorModule::Get();

		bool bTagRemoved = false;
		if (GameplayTagPickerMode == EGameplayTagPickerMode::HybridMode)
		{
			for (FGameplayTagContainer& Container : TagContainers)
			{
				bTagRemoved |= Container.RemoveTag(InTagNode->GetCompleteTag());
			}
		}

		const bool bDeleted = TagsEditor.DeleteTagFromINI(InTagNode);

		if (bDeleted || bTagRemoved)
		{
			OnTagChanged.ExecuteIfBound(TagContainers);
		}
	}
	
	if (OwnerCombo.IsValid())
	{
		OwnerCombo->SetIsOpen(false);
	}
}

void SGameplayTagPicker::OnSelectExactTag(TSharedPtr<FGameplayTagNode> InTagNode, TSharedPtr<SComboButton> OwnerCombo)
{
	FScopedTransaction Transaction(LOCTEXT("GameplayTagPicker_SelectTags", "Select Gameplay Tags"));

	if (InTagNode.IsValid())
	{
		for (FGameplayTagContainer& Container : TagContainers)
		{
			Container.AddTag(InTagNode->GetCompleteTag());
		}
	}

	OnContainersChanged();
	
	if (OwnerCombo.IsValid())
	{
		OwnerCombo->SetIsOpen(false);
	}
}

void SGameplayTagPicker::OnUnselectExactTag(TSharedPtr<FGameplayTagNode> InTagNode, TSharedPtr<SComboButton> OwnerCombo)
{
	FScopedTransaction Transaction(LOCTEXT("GameplayTagPicker_SelectTags", "Select Gameplay Tags"));

	if (InTagNode.IsValid())
	{
		for (FGameplayTagContainer& Container : TagContainers)
		{
			Container.RemoveTag(InTagNode->GetCompleteTag());
		}
	}

	OnContainersChanged();

	if (OwnerCombo.IsValid())
	{
		OwnerCombo->SetIsOpen(false);
	}
}

void SGameplayTagPicker::OnSearchForReferences(TSharedPtr<FGameplayTagNode> InTagNode, TSharedPtr<SComboButton> OwnerCombo)
{
	if (InTagNode.IsValid())
	{
		TArray<FAssetIdentifier> AssetIdentifiers;
		AssetIdentifiers.Add(FAssetIdentifier(FGameplayTag::StaticStruct(), InTagNode->GetCompleteTagName()));
		FEditorDelegates::OnOpenReferenceViewer.Broadcast(AssetIdentifiers, FReferenceViewerParams());
	}
	
	if (OwnerCombo.IsValid())
	{
		OwnerCombo->SetIsOpen(false);
	}
}

void SGameplayTagPicker::OnSearchForReferencesAsChildren(TSharedPtr<FGameplayTagNode> InTagNode, TSharedPtr<SComboButton> OwnerCombo)
{
	if (InTagNode.IsValid())
	{
		UGameplayTagsManager& TagManager = UGameplayTagsManager::Get();
		UScriptStruct* GameplayTagStruct = FGameplayTag::StaticStruct();

		TArray<FAssetIdentifier> AssetIdentifiers;
		AssetIdentifiers.Add(FAssetIdentifier(GameplayTagStruct, InTagNode->GetCompleteTagName()));

		FGameplayTagContainer TagContainer = TagManager.RequestGameplayTagChildrenInDictionary(InTagNode->GetCompleteTag());
		for (FGameplayTag ChildTag : TagContainer.GetGameplayTagArray())
		{
			FAssetIdentifier ChildAssetIdentifier(GameplayTagStruct, ChildTag.GetTagName());
			AssetIdentifiers.Add(ChildAssetIdentifier);
		}

		FEditorDelegates::OnOpenReferenceViewer.Broadcast(AssetIdentifiers, FReferenceViewerParams());
	}

	if (OwnerCombo.IsValid())
	{
		OwnerCombo->SetIsOpen(false);
	}
}

void SGameplayTagPicker::OnCopyTagNameToClipboard(TSharedPtr<FGameplayTagNode> InTagNode, TSharedPtr<SComboButton> OwnerCombo)
{
	if (InTagNode.IsValid())
	{
		const FString TagName = InTagNode->GetCompleteTagString();
		FPlatformApplicationMisc::ClipboardCopy(*TagName);
	}
	
	if (OwnerCombo.IsValid())
	{
		OwnerCombo->SetIsOpen(false);
	}
}

void SGameplayTagPicker::SetTagTreeItemExpansion(bool bExpand, bool bPersistExpansion)
{
	TArray<TSharedPtr<FGameplayTagNode>> TagArray;
	UGameplayTagsManager::Get().GetFilteredGameplayRootTags(TEXT(""), TagArray);
	for (int32 TagIdx = 0; TagIdx < TagArray.Num(); ++TagIdx)
	{
		SetTagNodeItemExpansion(TagArray[TagIdx], bExpand, bPersistExpansion);
	}
}

void SGameplayTagPicker::SetTagNodeItemExpansion(TSharedPtr<FGameplayTagNode> Node, bool bExpand, bool bPersistExpansion)
{
	TGuardValue<bool> PersistExpansionChangeGuard(bPersistExpansionChange, bPersistExpansion);
	if (Node.IsValid() && TagTreeWidget.IsValid())
	{
		TagTreeWidget->SetItemExpansion(Node, bExpand);

		const TArray<TSharedPtr<FGameplayTagNode>>& ChildTags = Node->GetChildTagNodes();
		for (int32 ChildIdx = 0; ChildIdx < ChildTags.Num(); ++ChildIdx)
		{
			SetTagNodeItemExpansion(ChildTags[ChildIdx], bExpand, bPersistExpansion);
		}
	}
}

void SGameplayTagPicker::LoadSettings()
{
	MigrateSettings();

	CachedExpandedItems.Reset();
	TArray<TSharedPtr<FGameplayTagNode>> TagArray;
	UGameplayTagsManager::Get().GetFilteredGameplayRootTags(TEXT(""), TagArray);
	for (int32 TagIdx = 0; TagIdx < TagArray.Num(); ++TagIdx)
	{
		LoadTagNodeItemExpansion(TagArray[TagIdx]);
	}
}

const FString& SGameplayTagPicker::GetGameplayTagsEditorStateIni()
{
	static FString Filename;

	if (Filename.Len() == 0)
	{
		Filename = FConfigCacheIni::NormalizeConfigIniPath(FString::Printf(TEXT("%s%hs/GameplayTagsEditorState.ini"), *FPaths::GeneratedConfigDir(), FPlatformProperties::PlatformName()));
	}

	return Filename;
}

void SGameplayTagPicker::MigrateSettings()
{
	if (const FConfigSection* EditorPerProjectIniSection = GConfig->GetSection(*SettingsIniSection, /*Force=*/false, GEditorPerProjectIni))
	{
		if (EditorPerProjectIniSection->Num() > 0)
		{
			const FString& DestFilename = GetGameplayTagsEditorStateIni();
			for (const auto& It : *EditorPerProjectIniSection)
			{
				GConfig->AddUniqueToSection(*SettingsIniSection, It.Key, It.Value.GetSavedValue(), DestFilename);
			}

			GConfig->Flush(false, DestFilename);
		}

		GConfig->EmptySection(*SettingsIniSection, GEditorPerProjectIni);
		GConfig->Flush(false, GEditorPerProjectIni);
	}
}

void SGameplayTagPicker::SetDefaultTagNodeItemExpansion(TSharedPtr<FGameplayTagNode> Node)
{
	TGuardValue<bool> PersistExpansionChangeGuard(bPersistExpansionChange, false);
	if (Node.IsValid() && TagTreeWidget.IsValid())
	{
		const bool bIsExpanded = CachedExpandedItems.Contains(Node) || IsTagChecked(Node) == ECheckBoxState::Checked || IsNodeSingleRoot(Node);

		TagTreeWidget->SetItemExpansion(Node, bIsExpanded);
		
		const TArray<TSharedPtr<FGameplayTagNode>>& ChildTags = Node->GetChildTagNodes();
		for (int32 ChildIdx = 0; ChildIdx < ChildTags.Num(); ++ChildIdx)
		{
			SetDefaultTagNodeItemExpansion(ChildTags[ChildIdx]);
		}
	}
}

void SGameplayTagPicker::LoadTagNodeItemExpansion(TSharedPtr<FGameplayTagNode> Node)
{
	TGuardValue<bool> PersistExpansionChangeGuard(bPersistExpansionChange, false);
	if (Node.IsValid() && TagTreeWidget.IsValid())
	{
		bool bIsExpanded = false;

		if (IsNodeSingleRoot(Node)) // Single roots are always expanded
		{
			TagTreeWidget->SetItemExpansion(Node, true);
		}
		else if (GConfig->GetBool(*SettingsIniSection, *(SettingsName + Node->GetCompleteTagString() + TEXT(".Expanded")), bIsExpanded, GetGameplayTagsEditorStateIni()))
		{
			TagTreeWidget->SetItemExpansion(Node, bIsExpanded);
			if (bIsExpanded)
			{
				CachedExpandedItems.Add(Node);
			}
		}
		else if (IsTagChecked(Node) == ECheckBoxState::Checked) // If we have no save data but its ticked then we probably lost our settings so we shall expand it
		{
			TagTreeWidget->SetItemExpansion(Node, true);
		}

		const TArray<TSharedPtr<FGameplayTagNode>>& ChildTags = Node->GetChildTagNodes();
		for (int32 ChildIdx = 0; ChildIdx < ChildTags.Num(); ++ChildIdx)
		{
			LoadTagNodeItemExpansion(ChildTags[ChildIdx]);
		}
	}
}

void SGameplayTagPicker::OnExpansionChanged(TSharedPtr<FGameplayTagNode> InItem, bool bIsExpanded)
{
	if (bPersistExpansionChange)
	{
		// Save the new expansion setting to ini file
		GConfig->SetBool(*SettingsIniSection, *(SettingsName + InItem->GetCompleteTagString() + TEXT(".Expanded")), bIsExpanded, GetGameplayTagsEditorStateIni());

		if (bIsExpanded)
		{
			CachedExpandedItems.Add(InItem);
		}
		else
		{
			CachedExpandedItems.Remove(InItem);
		}
	}
}

void SGameplayTagPicker::OnContainersChanged()
{
	if (PropertyHandle.IsValid() && bMultiSelect)
	{
		// Case for a tag container
		TArray<FString> PerObjectValues;
		for (const FGameplayTagContainer& Container : TagContainers)
		{
			PerObjectValues.Push(Container.ToString());
		}
		PropertyHandle->SetPerObjectValues(PerObjectValues);
	}
	else if (PropertyHandle.IsValid() && !bMultiSelect)
	{
		// Case for a single Tag		
		FString FormattedString = TEXT("(TagName=\"");
		FormattedString += TagContainers[0].First().GetTagName().ToString();
		FormattedString += TEXT("\")");
		PropertyHandle->SetValueFromFormattedString(FormattedString);
	}

	OnTagChanged.ExecuteIfBound(TagContainers);
}

void SGameplayTagPicker::OnGameplayTagAdded(const FString& TagName, const FString& TagComment, const FName& TagSource)
{
	UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

	// Make sure the new tag is visible.
	TSharedPtr<FGameplayTagNode> TagNode = Manager.FindTagNode(FName(*TagName));
	TSharedPtr<FGameplayTagNode> ParentTagNode = TagNode;
	while (ParentTagNode.IsValid())
	{
		const FString Key = SettingsName + ParentTagNode->GetCompleteTagString() + TEXT(".Expanded");
		GConfig->SetBool(*SettingsIniSection, *Key, true, GetGameplayTagsEditorStateIni());
		CachedExpandedItems.Add(ParentTagNode);
		
		ParentTagNode = ParentTagNode->GetParentTagNode();
	}

	RefreshTags();
	TagTreeWidget->RequestTreeRefresh();

	if (TagNode.IsValid())
	{
		TGuardValue<bool> PersistExpansionChangeGuard(bInSelectionChanged, true);

		TagTreeWidget->ClearSelection();
		TagTreeWidget->SetItemSelection(TagNode, true);
		
		if (TagNode->CompleteTagWithParents.Num() > 0)
		{
			RequestScrollToView(TagNode->CompleteTagWithParents.GetByIndex(0));
		}
	}
}

void SGameplayTagPicker::RefreshTags()
{
	UGameplayTagsManager& Manager = UGameplayTagsManager::Get();
	Manager.GetFilteredGameplayRootTags(RootFilterString, TagItems);

	if (bRestrictedTags)
	{
		// We only want to show the restricted gameplay tags
		for (int32 Idx = TagItems.Num() - 1; Idx >= 0; --Idx)
		{
			if (!TagItems[Idx]->IsRestrictedGameplayTag())
			{
				TagItems.RemoveAtSwap(Idx);
			}
		}
	}

	if (Manager.OnFilterGameplayTag.IsBound())
	{
		for (int32 Idx = TagItems.Num() - 1; Idx >= 0; --Idx)
		{
			bool DelegateShouldHide = false;
			FGameplayTagSource* Source = Manager.FindTagSource(TagItems[Idx]->GetFirstSourceName());
			Manager.OnFilterGameplayTag.Broadcast(UGameplayTagsManager::FFilterGameplayTagContext(RootFilterString, TagItems[Idx], Source, PropertyHandle), DelegateShouldHide);
			if (DelegateShouldHide)
			{
				TagItems.RemoveAtSwap(Idx);
			}
		}
	}

	// Restore expansion state.
	CachedExpandedItems.Reset();
	for (int32 TagIdx = 0; TagIdx < TagItems.Num(); ++TagIdx)
	{
		LoadTagNodeItemExpansion(TagItems[TagIdx]);
	}

	FilterTagTree();

	OnRefreshTagContainers.ExecuteIfBound(*this);
}

EVisibility SGameplayTagPicker::DetermineExpandableUIVisibility() const
{
	const UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

	if (!Manager.ShouldImportTagsFromINI() || GameplayTagPickerMode == EGameplayTagPickerMode::SelectionMode)
	{
		// If we can't support adding tags from INI files, or both options are forcibly disabled, we should never see this widget
		return EVisibility::Collapsed;
	}

	return EVisibility::Visible;
}


bool SGameplayTagPicker::CanAddNewTag() const
{
	const UGameplayTagsManager& Manager = UGameplayTagsManager::Get();
	return !bReadOnly && Manager.ShouldImportTagsFromINI();
}

bool SGameplayTagPicker::CanAddNewSubTag(TSharedPtr<FGameplayTagNode> Node) const
{
	if (!Node.IsValid())
	{
		return false;
	}
	
	if (!CanAddNewTag())
	{
		return false;
	}

	// We do not allow you to add child tags under a conflict
	if (Node->bNodeHasConflict || Node->bAncestorHasConflict)
	{
		return false;
	}

	// show if we're dealing with restricted tags exclusively or restricted tags that allow non-restricted children
	if (Node->GetAllowNonRestrictedChildren() || bRestrictedTags)
	{
		return true;
	}

	return false;
}

bool SGameplayTagPicker::CanSelectTags() const
{
	return !bReadOnly
			&& (GameplayTagPickerMode == EGameplayTagPickerMode::SelectionMode
				|| GameplayTagPickerMode == EGameplayTagPickerMode::HybridMode);
}

void SGameplayTagPicker::RefreshOnNextTick()
{
	bDelayRefresh = true;
}

void SGameplayTagPicker::RequestScrollToView(const FGameplayTag RequestedTag)
{
	RequestedScrollToTag = RequestedTag;
}

void SGameplayTagPicker::OnGameplayTagRenamed(FString OldTagName, FString NewTagName)
{
	// @todo: replace changed tag?
	OnTagChanged.ExecuteIfBound(TagContainers);
}

void SGameplayTagPicker::OpenRenameGameplayTagDialog(TSharedPtr<FGameplayTagNode> GameplayTagNode) const
{
	TSharedRef<SWindow> RenameTagWindow =
		SNew(SWindow)
		.Title(LOCTEXT("RenameTagWindowTitle", "Rename Gameplay Tag"))
		.SizingRule(ESizingRule::Autosized)
		.SupportsMaximize(false)
		.SupportsMinimize(false);

	TSharedRef<SRenameGameplayTagDialog> RenameTagDialog =
		SNew(SRenameGameplayTagDialog)
		.GameplayTagNode(GameplayTagNode)
		.OnGameplayTagRenamed(const_cast<SGameplayTagPicker*>(this), &SGameplayTagPicker::OnGameplayTagRenamed);

	RenameTagWindow->SetContent(SNew(SBox)
		.MinDesiredWidth(320.0f)
		[
			RenameTagDialog
		]);

	TSharedPtr<SWindow> CurrentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared() );

	FSlateApplication::Get().AddModalWindow(RenameTagWindow, CurrentWindow);
}

TSharedPtr<SWidget> SGameplayTagPicker::GetWidgetToFocusOnOpen()
{
	return SearchTagBox;
}

void SGameplayTagPicker::SetTagContainers(TConstArrayView<FGameplayTagContainer> InTagContainers)
{
	TagContainers = InTagContainers;
}

void SGameplayTagPicker::OnPostUndoRedo()
{
	OnRefreshTagContainers.ExecuteIfBound(*this);
}

void SGameplayTagPicker::VerifyAssetTagValidity()
{
	UGameplayTagsManager& TagsManager = UGameplayTagsManager::Get();

	// Find and remove any tags on the asset that are no longer in the library
	bool bChanged = false;
	for (FGameplayTagContainer& Container : TagContainers)
	{
		// Use a set instead of a container so we can find and remove None tags
		TSet<FGameplayTag> InvalidTags;

		for (auto It = Container.CreateConstIterator(); It; ++It)
		{
			const FGameplayTag TagToCheck = *It;

			if (!UGameplayTagsManager::Get().RequestGameplayTag(TagToCheck.GetTagName(), false).IsValid())
			{
				InvalidTags.Add(*It);
			}
		}

		if (InvalidTags.Num() > 0)
		{
			FString InvalidTagNames;

			for (auto InvalidIter = InvalidTags.CreateConstIterator(); InvalidIter; ++InvalidIter)
			{
				Container.RemoveTag(*InvalidIter);
				InvalidTagNames += InvalidIter->ToString() + TEXT("\n");
			}
			
			bChanged = true;

			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("Objects"), FText::FromString(InvalidTagNames));
			FText DialogText = FText::Format(LOCTEXT("GameplayTagPicker_InvalidTags", "Invalid Tags that have been removed: \n\n{Objects}"), Arguments);
			FMessageDialog::Open(EAppMsgType::Ok, DialogText, LOCTEXT("GameplayTagPicker_Warning", "Warning"));
		}
	}

	if (bChanged)
	{
		OnContainersChanged();
	}
}

void SGameplayTagPicker::GetFilteredGameplayRootTags(const FString& InFilterString, TArray<TSharedPtr<FGameplayTagNode>>& OutNodes) const
{
	OutNodes.Empty();
	const UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

	if (TagFilter.IsBound())
	{
		TArray<TSharedPtr<FGameplayTagNode>> UnfilteredItems;
		Manager.GetFilteredGameplayRootTags(InFilterString, UnfilteredItems);
		for (const TSharedPtr<FGameplayTagNode>& Node : UnfilteredItems)
		{
			if (TagFilter.Execute(Node) == ETagFilterResult::IncludeTag)
			{
				OutNodes.Add(Node);
			}
		}
	}
	else
	{
		Manager.GetFilteredGameplayRootTags(InFilterString, OutNodes);
	}
}

namespace UE::GameplayTags::Editor
{

static TWeakPtr<SGameplayTagPicker> GlobalTagWidget;
static TWeakPtr<SWindow> GlobalTagWidgetWindow;

void CloseGameplayTagWindow(TWeakPtr<SGameplayTagPicker> TagWidget)
{
	if (GlobalTagWidget.IsValid() && GlobalTagWidgetWindow.IsValid())
	{
		if (!TagWidget.IsValid() || TagWidget == GlobalTagWidget)
		{
			GlobalTagWidgetWindow.Pin()->RequestDestroyWindow();
		}
	}
	
	GlobalTagWidgetWindow = nullptr;
	GlobalTagWidget = nullptr;
}

TSharedRef<SGameplayTagPicker> Create(const FGameplayTagManagerWindowArgs& Args)
{
	return SNew(SGameplayTagPicker)
		.Filter(Args.Filter)
		.ReadOnly(false)
		.MaxHeight(0.0f) // unbounded
		.MultiSelect(false)
		.SettingsName(TEXT("Manager"))
		.GameplayTagPickerMode(EGameplayTagPickerMode::ManagementMode)
		.RestrictedTags(Args.bRestrictedTags)
		;
}


TWeakPtr<SGameplayTagPicker> OpenGameplayTagManager(const FGameplayTagManagerWindowArgs& Args)
{
	TSharedPtr<SWindow> GameplayTagPickerWindow = GlobalTagWidgetWindow.Pin();
	TSharedPtr<SGameplayTagPicker> TagWidget = GlobalTagWidget.Pin();
	
	if (!GlobalTagWidgetWindow.IsValid()
		|| !GlobalTagWidget.IsValid())
	{
		// Close all other GameplayTag windows.
		CloseGameplayTagWindow(nullptr);


		TagWidget = Create(Args);

		FText Title = Args.Title;
		if (Title.IsEmpty())
		{
			if (Args.bRestrictedTags)
			{
				Title = LOCTEXT("GameplayTagPicker_RestrictedTitle", "Restricted Gameplay Tags");
			}
			else
			{
				Title = LOCTEXT("GameplayTagPicker_ManagerTitle", "Gameplay Tag Manager");
			}
		}

		const FVector2D WindowSize(800, 800);
		GameplayTagPickerWindow = SNew(SWindow)
			.Title(Title)
			.HasCloseButton(true)
			.SupportsMaximize(false)
			.SupportsMinimize(false)
			.AutoCenter(EAutoCenter::PreferredWorkArea)
			.ClientSize(WindowSize)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
					.FillHeight(1)
					[
						TagWidget.ToSharedRef()
					]
				]
			];
		
		// NOTE: FGlobalTabmanager::Get()-> is actually dereferencing a SharedReference, not a SharedPtr, so it cannot be null.
		if (FGlobalTabmanager::Get()->GetRootWindow().IsValid())
		{
			FSlateApplication::Get().AddWindowAsNativeChild(GameplayTagPickerWindow.ToSharedRef(), FGlobalTabmanager::Get()->GetRootWindow().ToSharedRef());
		}
		else
		{
			FSlateApplication::Get().AddWindow(GameplayTagPickerWindow.ToSharedRef());
		}

		GlobalTagWidget = TagWidget;
		GlobalTagWidgetWindow = GameplayTagPickerWindow;
	}

	check (TagWidget.IsValid());

	// Set focus to the search box on creation
	FSlateApplication::Get().SetKeyboardFocus(TagWidget->GetWidgetToFocusOnOpen());
	FSlateApplication::Get().SetUserFocus(0, TagWidget->GetWidgetToFocusOnOpen());

	if (Args.HighlightedTag.IsValid())
	{
		TagWidget->RequestScrollToView(Args.HighlightedTag);
	}

	return TagWidget;
}

}

#undef LOCTEXT_NAMESPACE
