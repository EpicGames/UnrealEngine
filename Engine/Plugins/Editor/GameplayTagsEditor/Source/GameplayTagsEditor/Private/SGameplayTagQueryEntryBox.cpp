// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGameplayTagQueryEntryBox.h"
#include "DetailLayoutBuilder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "ScopedTransaction.h"
#include "SGameplayTagQueryWidget.h"
#include "Editor.h"
#include "GameplayTagEditorUtilities.h"
#include "GameplayTagsManager.h"

#define LOCTEXT_NAMESPACE "GameplayTagContainerCombo"

//------------------------------------------------------------------------------
// SGameplayTagQueryEntryBox
//------------------------------------------------------------------------------

SLATE_IMPLEMENT_WIDGET(SGameplayTagQueryEntryBox)
void SGameplayTagQueryEntryBox::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "TagQuery", TagQueryAttribute, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateLambda([](SWidget& Widget)
			{
				static_cast<SGameplayTagQueryEntryBox&>(Widget).CacheQueryList();
			}));
}

SGameplayTagQueryEntryBox::SGameplayTagQueryEntryBox()
	: TagQueryAttribute(*this)
{
}

void SGameplayTagQueryEntryBox::Construct(const FArguments& InArgs)
{
	Filter = InArgs._Filter;
	bIsReadOnly = InArgs._ReadOnly;
	OnTagQueryChanged = InArgs._OnTagQueryChanged;
	PropertyHandle = InArgs._PropertyHandle;

	CacheQueryList();

	if (PropertyHandle.IsValid())
	{
		PropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &SGameplayTagQueryEntryBox::CacheQueryList));
		bIsReadOnly = PropertyHandle->IsEditConst();

		if (Filter.IsEmpty())
		{
			Filter = UGameplayTagsManager::Get().GetCategoriesMetaFromPropertyHandle(PropertyHandle);
		}

		// Let's check if there's already an open editor window for this property and rebuild that
		QueryWidget = UE::GameplayTags::Editor::TrySyncGameplayTagQueryWidget(BuildWindowArgs());
	}
	else
	{
		TagQueryAttribute.Assign(*this, InArgs._TagQuery);
	}

	ChildSlot
	[
		SAssignNew(WidgetContainer, SHorizontalBox)

		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.FillWidth(1)
		.MaxWidth(InArgs._DescriptionMaxWidth)
		[
			SNew(SButton)
			.IsEnabled(this, &SGameplayTagQueryEntryBox::IsValueEnabled)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.OnClicked(this, &SGameplayTagQueryEntryBox::OnEditButtonClicked)
			[
				SNew(STextBlock)
				.Text(this, &SGameplayTagQueryEntryBox::GetQueryDescText)
				.Font(FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont")))
				.ToolTipText(this, &SGameplayTagQueryEntryBox::GetQueryDescTooltip)
				.Clipping(EWidgetClipping::OnDemand)
				.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
				.AutoWrapText(false)
			]
		]

		// Edit query
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Top)
		[
			SNew(SButton)
			.IsEnabled(this, &SGameplayTagQueryEntryBox::IsValueEnabled)
			.ToolTipText(LOCTEXT("GameplayTagQueryEntryBox_Edit", "Edit Gameplay Tag Query."))
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.OnClicked(this, &SGameplayTagQueryEntryBox::OnEditButtonClicked)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.Edit"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]

		// Clear query
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Top)
		[
			SNew(SButton)
			.IsEnabled(this, &SGameplayTagQueryEntryBox::IsValueEnabled)
			.ToolTipText(LOCTEXT("GameplayTagQueryEntryBox_Clear", "Clear Query"))
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.OnClicked(this, &SGameplayTagQueryEntryBox::OnClearAllButtonClicked)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.Delete"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]
	];
}

FText SGameplayTagQueryEntryBox::GetQueryDescText() const
{
	return QueryDescription;
}

FText SGameplayTagQueryEntryBox::GetQueryDescTooltip() const
{
	return QueryDescriptionTooltip;
}

FReply SGameplayTagQueryEntryBox::OnClearAllButtonClicked()
{
	if (PropertyHandle.IsValid())
	{
		const FScopedTransaction Transaction( FText::Format(LOCTEXT("GameplayTagQueryEntryBox_ClearQuery", "Clear Query"), PropertyHandle->GetPropertyDisplayName()));
		FString EmptyQueryAsString;
    	FGameplayTagQuery::StaticStruct()->ExportText(EmptyQueryAsString, &FGameplayTagQuery::EmptyQuery, &FGameplayTagQuery::EmptyQuery, /*OwnerObject*/nullptr, PPF_None, /*ExportRootScope*/nullptr);
    	PropertyHandle->SetValueFromFormattedString(EmptyQueryAsString);
	}

	// Update for attribute version and callbacks.
	for (FGameplayTagQuery& Query : CachedQueries)
	{
		Query = FGameplayTagQuery::EmptyQuery;
	}

	if (!CachedQueries.IsEmpty())
	{
		OnTagQueryChanged.ExecuteIfBound(CachedQueries[0]);
	}

	CacheQueryList();

	return FReply::Handled();
}

bool SGameplayTagQueryEntryBox::HasAnyValidQueries() const
{
	for (const FGameplayTagQuery& TagQuery : CachedQueries)
	{
		if (!TagQuery.IsEmpty())
		{
			return true;
		}
	}
	return false;
}

bool SGameplayTagQueryEntryBox::IsValueEnabled() const
{
	if (PropertyHandle.IsValid())
	{
		return !PropertyHandle->IsEditConst();
	}

	return !bIsReadOnly;
}

EVisibility SGameplayTagQueryEntryBox::GetQueryDescVisibility() const
{
	return HasAnyValidQueries() == true ? EVisibility::Visible : EVisibility::Collapsed;
}

FGameplayTagQueryWindowArgs SGameplayTagQueryEntryBox::BuildWindowArgs()
{
	FGameplayTagQueryWindowArgs Args;
	Args.OnQueriesCommitted = SGameplayTagQueryWidget::FOnQueriesCommitted::CreateSP(this, &SGameplayTagQueryEntryBox::OnQueriesCommitted);
	Args.EditableQueries = CachedQueries;
	Args.AnchorWidget = WidgetContainer;
	Args.bReadOnly = !IsValueEnabled();
	Args.Filter = Filter;
	
	if (PropertyHandle.IsValid())
	{
		Args.Property = PropertyHandle->GetProperty();

		TArray<UObject*> OuterObjects;
		PropertyHandle->GetOuterObjects(OuterObjects);

		if (OuterObjects.IsEmpty())
		{
			TArray<UPackage*> OuterPackages;
			PropertyHandle->GetOuterPackages(OuterPackages);
			for (UPackage* Package : OuterPackages)
			{
				OuterObjects.Add(Package);
			}
		}
		
		if (OuterObjects.Num() > 1)
		{
			FText const AssetName = FText::Format(LOCTEXT("GameplayTagDetailsBase_MultipleAssets", "{0} Assets"), FText::AsNumber(OuterObjects.Num()));
			FText const PropertyName = PropertyHandle->GetPropertyDisplayName();
			Args.Title = FText::Format(LOCTEXT("GameplayTagQueryEntryBox_BaseWidgetTitle", "Tag Query Editor: {0} {1}"), PropertyName, AssetName);
		}
		else if (OuterObjects.Num() > 0 && OuterObjects[0])
		{
			FText const AssetName = FText::FromString(OuterObjects[0]->GetName());
			FText const PropertyName = PropertyHandle->GetPropertyDisplayName();
			Args.Title = FText::Format(LOCTEXT("GameplayTagQueryEntryBox_BaseWidgetTitle", "Tag Query Editor: {0} {1}"), PropertyName, AssetName);
		}
	}
	else
	{
		Args.Title = LOCTEXT("GameplayTagQueryEntryBox_WidgetTitle", "Tag Editor");
	}

	return Args;
}

FReply SGameplayTagQueryEntryBox::OnEditButtonClicked()
{
	QueryWidget = UE::GameplayTags::Editor::OpenGameplayTagQueryWindow(BuildWindowArgs());
	
	return FReply::Handled();
}

void SGameplayTagQueryEntryBox::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (!PropertyHandle.IsValid()
		|| !PropertyHandle->IsValidHandle())
	{
		return;
	}

	// Check if cached data has changed, and update it.
	bool bShouldUpdate = false;
	
	TArray<const void*> RawStructData;
	PropertyHandle->AccessRawData(RawStructData);

	if (RawStructData.Num() == CachedQueries.Num())
	{
		for (int32 Idx = 0; Idx < RawStructData.Num(); ++Idx)
		{
			if (RawStructData[Idx])
			{
				const FGameplayTagQuery& Query = *(FGameplayTagQuery*)RawStructData[Idx];
				if (Query != CachedQueries[Idx])
				{
					bShouldUpdate = true;
					break;
				}
			}
		}
	}

	if (bShouldUpdate)
	{
		CacheQueryList();
	}
}

void SGameplayTagQueryEntryBox::CacheQueryList()
{
	CachedQueries.Empty();

	if (PropertyHandle.IsValid())
	{
		if (PropertyHandle->IsValidHandle())
		{
			// Cache queries from the property handle. Add empty queries even if the instance data is null so that the indices match with the property handle.
			TArray<void*> RawStructData;
			PropertyHandle->AccessRawData(RawStructData);
			
			for (int32 Idx = 0; Idx < RawStructData.Num(); ++Idx)
			{
				FGameplayTagQuery& Query = CachedQueries.AddDefaulted_GetRef();
				if (RawStructData[Idx])
				{
					Query = *(FGameplayTagQuery*)RawStructData[Idx]; 
				}
			}
		}
	}
	else
	{
		CachedQueries.Add(TagQueryAttribute.Get());
	}

	// Cache the description
	QueryDescription = LOCTEXT("GameplayTagQueryEntryBox_EmptyQuery", "Empty");
	QueryDescriptionTooltip = LOCTEXT("GameplayTagQueryEntryBox_EmptyQueryTooltip", "Empty Gameplay Tag Query");

	bool bAllSame = true;
	for (int32 Index = 1; Index < CachedQueries.Num(); Index++)
	{
		if (CachedQueries[Index] != CachedQueries[0])
		{
			bAllSame = false;
			break;
		}
	}
	
	if (!bAllSame)
	{
		QueryDescription = LOCTEXT("GameplayTagQueryEntryBox_MultipleSelected", "Multiple Selected");
		QueryDescriptionTooltip = QueryDescription;
	}
	else if (CachedQueries.Num() == 1)
	{
		const FGameplayTagQuery& TheQuery = CachedQueries[0];
		FString Desc = TheQuery.GetDescription();

		// We can get into a case where we've manually generated an FGameplayTagQuery but do not have a proper description
		// in this case, regenerate a description for the query and use that until next time it's edited.
		if (Desc.IsEmpty() && !TheQuery.IsEmpty())
		{
			UEditableGameplayTagQuery* EditableQuery = TheQuery.CreateEditableQuery();

			FGameplayTagQuery TempQueryForDescription;
			TempQueryForDescription.BuildFromEditableQuery(*EditableQuery);
			Desc = TempQueryForDescription.GetDescription();
		}

		if (Desc.Len() > 0)
		{
			QueryDescription = FText::FromString(Desc);
			QueryDescriptionTooltip = FText::FromString(UE::GameplayTags::EditorUtilities::FormatGameplayTagQueryDescriptionToLines(Desc));
		}
	}
}

void SGameplayTagQueryEntryBox::OnQueriesCommitted(const TArray<FGameplayTagQuery>& TagQueries)
{
	// Notify change.
	if (PropertyHandle.IsValid())
	{
		const FScopedTransaction Transaction( FText::Format(LOCTEXT("GameplayTagQueryEntryBox_EditValue", "Edit {0}"), PropertyHandle->GetPropertyDisplayName()));

		PropertyHandle->NotifyPreChange();

		TArray<FString> PerObjectValues;
		for (const FGameplayTagQuery& TagQuery : TagQueries)
		{
			FString& ExportString = PerObjectValues.AddDefaulted_GetRef();
			FGameplayTagQuery::StaticStruct()->ExportText(ExportString, &TagQuery, &TagQuery, nullptr, 0, nullptr);
		}
		PropertyHandle->SetPerObjectValues(PerObjectValues);

		PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		PropertyHandle->NotifyFinishedChangingProperties();
	}

	// Update for attribute version and callbacks.
	CachedQueries = TagQueries;

	if (!CachedQueries.IsEmpty())
	{
		OnTagQueryChanged.ExecuteIfBound(CachedQueries[0]);
	}
	
	CacheQueryList();
}

#undef LOCTEXT_NAMESPACE
