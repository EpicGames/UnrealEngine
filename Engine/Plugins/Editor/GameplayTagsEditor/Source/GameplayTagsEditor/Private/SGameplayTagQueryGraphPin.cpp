// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGameplayTagQueryGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "GameplayTagContainer.h"
#include "ScopedTransaction.h"
#include "GameplayTagEditorUtilities.h"
#include "SGameplayTagQueryEntryBox.h"

#define LOCTEXT_NAMESPACE "GameplayTagQueryGraphPin"


void SGameplayTagQueryGraphPin::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPin::Construct( SGraphPin::FArguments(), InGraphPinObj );
}

void SGameplayTagQueryGraphPin::OnTagQueryChanged(const FGameplayTagQuery& NewTagQuery)
{
	TagQuery = NewTagQuery;
			
	const FString TagQueryExportText = UE::GameplayTags::EditorUtilities::GameplayTagQueryExportText(TagQuery);

	if (TagQueryExportText != GraphPinObj->GetDefaultAsString())
	{
		// Set Pin Data
		const FScopedTransaction Transaction(NSLOCTEXT("GraphEditor", "ChangePinValue", "Change Pin Value"));
		GraphPinObj->Modify();
		GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, TagQueryExportText);
	}
}

TSharedRef<SWidget>	SGameplayTagQueryGraphPin::GetDefaultValueWidget()
{
	ParseDefaultValueData();

	return SNew(SGameplayTagQueryEntryBox)
		.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
		.Filter(UE::GameplayTags::EditorUtilities::ExtractTagFilterStringFromGraphPin(GraphPinObj))
		.DescriptionMaxWidth(250.0f)
		.TagQuery(this, &SGameplayTagQueryGraphPin::GetTagQuery)
		.OnTagQueryChanged(this, &SGameplayTagQueryGraphPin::OnTagQueryChanged);
}

FGameplayTagQuery SGameplayTagQueryGraphPin::GetTagQuery() const
{
	return TagQuery;
}

void SGameplayTagQueryGraphPin::ParseDefaultValueData()
{
	FString const TagQueryString = GraphPinObj->GetDefaultAsString();
	TagQuery = UE::GameplayTags::EditorUtilities::GameplayTagQueryTryImportText(TagQueryString);
}

#undef LOCTEXT_NAMESPACE
