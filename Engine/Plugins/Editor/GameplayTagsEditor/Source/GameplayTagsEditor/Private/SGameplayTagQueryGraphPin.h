// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphPin.h"
#include "GameplayTagContainer.h"

class SComboButton;

class SGameplayTagQueryGraphPin : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SGameplayTagQueryGraphPin) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

	//~ Begin SGraphPin Interface
	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	//~ End SGraphPin Interface

private:
	/** Parses the Data from the pin to fill in the names of the array. */
	void ParseDefaultValueData();

	FGameplayTagQuery GetTagQuery() const;
	void OnTagQueryChanged(const FGameplayTagQuery& NewTagQuery);

	/** Parse tag query used for editing. */
	FGameplayTagQuery TagQuery;
};
