// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGameplayTagQueryWidget.h"
#include "DetailsViewArgs.h"
#include "Modules/ModuleManager.h"
#include "PropertyHandle.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"
#include "IDetailsView.h"
#include "GameplayTagsManager.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "ScopedTransaction.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"

#define LOCTEXT_NAMESPACE "GameplayTagQueryWidget"

void SGameplayTagQueryWidget::Construct(const FArguments& InArgs, const TArray<FGameplayTagQuery>& InTagQueries)
{
	ensure(InTagQueries.Num() > 0);
	TagQueries = InTagQueries;

	bReadOnly = InArgs._ReadOnly;
	OnQueriesCommitted = InArgs._OnQueriesCommitted;
	Filter = InArgs._Filter;

	UGameplayTagsManager::Get().OnGetCategoriesMetaFromPropertyHandle.AddSP(this, &SGameplayTagQueryWidget::OnGetCategoriesMetaFromPropertyHandle);

	// build editable query object tree from the runtime query data
	UEditableGameplayTagQuery* const EditableGameplayTagQuery = CreateEditableQuery(TagQueries[0]);
	EditableQuery = EditableGameplayTagQuery;

	OriginalTagQueries = TagQueries;
	
	// create details view for the editable query object
	FDetailsViewArgs ViewArgs;
	ViewArgs.bAllowSearch = false;
	ViewArgs.NotifyHook = this;
	ViewArgs.bHideSelectionTip = true;
	ViewArgs.bShowObjectLabel = false;

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	Details = PropertyModule.CreateDetailView(ViewArgs);
	Details->SetObject(EditableGameplayTagQuery);
	Details->SetEnabled(TAttribute<bool>::Create(
		TAttribute<bool>::FGetter::CreateSP(this, &SGameplayTagQueryWidget::IsBoundToWindow)
	));

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SVerticalBox)

			// to delete!
			+ SVerticalBox::Slot()
			.FillHeight(1)
			.Padding(2)
			[
				Details.ToSharedRef()
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(5)
			[
				SNew(SUniformGridPanel)
				.MinDesiredSlotHeight(FAppStyle::Get().GetFloat("StandardDialog.MinDesiredSlotHeight"))
				.MinDesiredSlotWidth(FAppStyle::Get().GetFloat("StandardDialog.MinDesiredSlotWidth"))
				.SlotPadding(FAppStyle::Get().GetMargin("StandardDialog.SlotPadding"))

				+ SUniformGridPanel::Slot(0, 0)
				[
					// ok button
					SNew(SButton)
						.IsEnabled(!bReadOnly)
						.HAlign(HAlign_Center)
						.OnClicked(this, &SGameplayTagQueryWidget::OnOkClicked)
						.Text(LOCTEXT("GameplayTagQueryWidget_OK", "OK"))
				]

				+ SUniformGridPanel::Slot(1, 0)
				[
					// cancel button
					SNew(SButton)
						.IsEnabled(this, &SGameplayTagQueryWidget::IsBoundToWindow)
						.ContentPadding(FAppStyle::Get().GetMargin("StandardDialog.ContentPadding") )
						.HAlign(HAlign_Center)
						.OnClicked(this, &SGameplayTagQueryWidget::OnCancelClicked)
						.Text(LOCTEXT("GameplayTagQueryWidget_Cancel", "Cancel"))
				]
			]
		]
	];
	
}

void SGameplayTagQueryWidget::OnGetCategoriesMetaFromPropertyHandle(TSharedPtr<IPropertyHandle> PropertyHandle, FString& MetaString) const
{
	TArray<UObject*> OuterObjects;
	PropertyHandle->GetOuterObjects(OuterObjects);
	for (const UObject* Object : OuterObjects)
	{
		UObject* Outer = Object->GetOuter();
		while (Outer)
		{
			if (Outer == EditableQuery)
			{
				// This is us, re-route
				MetaString = Filter;
				return;
			}
			Outer = Outer->GetOuter();
		}
	}
}

UEditableGameplayTagQuery* SGameplayTagQueryWidget::CreateEditableQuery(FGameplayTagQuery& Q)
{
	UEditableGameplayTagQuery* const AnEditableQuery = Q.CreateEditableQuery();
	if (AnEditableQuery)
	{
		// prevent GC, will explicitly remove from root later
		AnEditableQuery->AddToRoot();
	}

	return AnEditableQuery;
}

bool SGameplayTagQueryWidget::IsBoundToWindow() const
{
	return OnQueriesCommitted.IsBound();
}

SGameplayTagQueryWidget::~SGameplayTagQueryWidget()
{
	// clean up our temp editing uobjects
	UEditableGameplayTagQuery* const TagQuery = EditableQuery.Get();
	if (TagQuery)
	{
		TagQuery->RemoveFromRoot();
	}
	
	UGameplayTagsManager::Get().OnGetCategoriesMetaFromPropertyHandle.RemoveAll(this);
}

void SGameplayTagQueryWidget::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	if (!bReadOnly)
	{
		FScopedTransaction Transaction( LOCTEXT("GameplayTagQueryWidget_Edit", "Edit Gameplay Tag Query") );
		SaveToTagQuery();
		OnQueriesCommitted.ExecuteIfBound(TagQueries);
	}
}

void SGameplayTagQueryWidget::SaveToTagQuery()
{
	if (!EditableQuery.IsValid() || bReadOnly)
	{
		return;
	}

	FGameplayTagQuery NewQuery;

	if (UEditableGameplayTagQuery* CurrentEditableQuery = EditableQuery.Get())
	{
		// Do not try to build empty queries.
		if (CurrentEditableQuery->RootExpression != nullptr)
		{
			NewQuery.BuildFromEditableQuery(*CurrentEditableQuery);
		}
	}

	for (FGameplayTagQuery& TagQuery : TagQueries)
	{
		TagQuery = NewQuery;
	}
}

FReply SGameplayTagQueryWidget::OnOkClicked()
{
	if (!bReadOnly)
	{
		FScopedTransaction Transaction( LOCTEXT("GameplayTagQueryWidget_Edit", "Edit Gameplay Tag Query") );
		
		SaveToTagQuery();

		OnQueriesCommitted.ExecuteIfBound(TagQueries);
		OnOk.ExecuteIfBound();
	}
	else
	{
		OnCancel.ExecuteIfBound();
	}
	
	return FReply::Handled();
}

FReply SGameplayTagQueryWidget::OnCancelClicked()
{
	if (TagQueries != OriginalTagQueries)
	{
		TagQueries = OriginalTagQueries;
		OnQueriesCommitted.ExecuteIfBound(TagQueries);	
	}
	OnCancel.ExecuteIfBound();
	
	return FReply::Handled();
}

namespace UE::GameplayTags::Editor
{

static TWeakPtr<SGameplayTagQueryWidget> GlobalTagQueryWidget;
static TWeakPtr<SWindow> GlobalTagQueryWidgetWindow;
static TWeakFieldPtr<FProperty> GlobalTagQueryWidgetProperty;

TWeakPtr<SGameplayTagQueryWidget> OpenGameplayTagQueryWindow(const FGameplayTagQueryWindowArgs& Args)
{
	CloseGameplayTagQueryWindow(nullptr);
	
	const FVector2D WindowSize(600, 400);
	
	FText Title = Args.Title;
	if (Title.IsEmpty())
	{
		Title = LOCTEXT("GameplayTagQueryWidget_Title", "Tag Query Editor");
	}

	// Determine the position of the window so that it will spawn near the mouse, but not go off the screen.
	FSlateRect AnchorRect;
	if (Args.AnchorWidget.IsValid())
	{
		AnchorRect = Args.AnchorWidget->GetCachedGeometry().GetLayoutBoundingRect();
	}
	else
	{
		AnchorRect = FSlateRect::FromPointAndExtent(FSlateApplication::Get().GetCursorPos(), FVector2d());
	}
	AnchorRect = AnchorRect.ExtendBy(FMargin(20));
	
	const FVector2D ProposedPlacement = AnchorRect.GetTopLeft() + FVector2D(-WindowSize.X, -WindowSize.Y * 0.2);
	const FVector2D AdjustedSummonLocation = FSlateApplication::Get().CalculatePopupWindowPosition(AnchorRect, WindowSize, true, ProposedPlacement, Orient_Vertical);

	TSharedRef<SGameplayTagQueryWidget> QueryWidget = SNew(SGameplayTagQueryWidget, Args.EditableQueries)
			.OnQueriesCommitted(Args.OnQueriesCommitted)
			.Filter(Args.Filter)
			.ReadOnly(Args.bReadOnly);

	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(Args.Title)
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		.AutoCenter(EAutoCenter::None)
		.IsTopmostWindow(true)
		.ScreenPosition(AdjustedSummonLocation)
		.ClientSize(WindowSize)
		[
			QueryWidget
		];

	// NOTE: FGlobalTabmanager::Get()-> is actually dereferencing a SharedReference, not a SharedPtr, so it cannot be null.
	if (FSlateApplication::Get().GetActiveModalWindow().IsValid())
	{
		FSlateApplication::Get().AddWindowAsNativeChild(Window, FSlateApplication::Get().GetActiveModalWindow().ToSharedRef());
	}
	else if (FGlobalTabmanager::Get()->GetRootWindow().IsValid())
	{
		FSlateApplication::Get().AddWindowAsNativeChild(Window, FGlobalTabmanager::Get()->GetRootWindow().ToSharedRef());
	}
	else
	{
		FSlateApplication::Get().AddWindow(Window);
	}

	GlobalTagQueryWidget = QueryWidget;
	GlobalTagQueryWidgetWindow = Window;
	GlobalTagQueryWidgetProperty = Args.Property;

	auto CloseWindow = [WeakQueryWidget = GlobalTagQueryWidget]()
	{
		CloseGameplayTagQueryWindow(WeakQueryWidget);
	};

	QueryWidget->SetOnOk(FSimpleDelegate::CreateLambda(CloseWindow));
	QueryWidget->SetOnCancel(FSimpleDelegate::CreateLambda(CloseWindow));

	return QueryWidget;
}

void CloseGameplayTagQueryWindow(TWeakPtr<SGameplayTagQueryWidget> QueryWidget)
{
	if (GlobalTagQueryWidget.IsValid() && GlobalTagQueryWidgetWindow.IsValid())
	{
		if (!QueryWidget.IsValid() || QueryWidget == GlobalTagQueryWidget)
		{
			GlobalTagQueryWidgetWindow.Pin()->RequestDestroyWindow();
		}
	}

	GlobalTagQueryWidget = nullptr;
	GlobalTagQueryWidgetWindow = nullptr;
	GlobalTagQueryWidgetProperty.Reset();
}

TWeakPtr<SGameplayTagQueryWidget> TrySyncGameplayTagQueryWidget(const FGameplayTagQueryWindowArgs& Args)
{
	// If the property does not match the currently open one, then there's nothing to sync
	if (!Args.Property.IsValid() || !Args.Property.HasSameIndexAndSerialNumber(GlobalTagQueryWidgetProperty))
	{
		return nullptr;
	}

	if (const TSharedPtr<SWindow> WidgetWindow = GlobalTagQueryWidgetWindow.Pin())
	{
		// If the window title is different, then it belongs to a different asset or property
		if (!WidgetWindow->GetTitle().EqualTo(Args.Title))
		{
			return nullptr;
		}
	}

	if (const TSharedPtr<SGameplayTagQueryWidget> QueryWidget = GlobalTagQueryWidget.Pin())
	{
		const bool bDetailsMatch = QueryWidget->IsReadOnly() == Args.bReadOnly
			&& QueryWidget->GetFilter() == Args.Filter
			&& QueryWidget->GetTagQueries() == Args.EditableQueries;

		// Only sync if properties are the same
		if (bDetailsMatch)
		{
			GlobalTagQueryWidgetProperty = Args.Property;
			QueryWidget->SetOnQueriesCommitted(Args.OnQueriesCommitted);
			return GlobalTagQueryWidget;
		}
	}

	return nullptr;
}
} // UE::GameplayTags::Editor

#undef LOCTEXT_NAMESPACE
