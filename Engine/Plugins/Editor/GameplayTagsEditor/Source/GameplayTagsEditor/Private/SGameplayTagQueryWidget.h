// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Misc/NotifyHook.h"
#include "UObject/WeakFieldPtr.h"

class IPropertyHandle;
struct FGameplayTagQuery;
class IDetailsView;
struct FPropertyChangedEvent;
class UEditableGameplayTagQuery;

/** Widget allowing user to tag assets with gameplay tags */
class SGameplayTagQueryWidget : public SCompoundWidget, public FNotifyHook
{
public:

	DECLARE_DELEGATE_OneParam( FOnQueriesCommitted, const TArray<FGameplayTagQuery>& /*TagQueries*/ )
	
	SLATE_BEGIN_ARGS( SGameplayTagQueryWidget )
		: _ReadOnly(false)
	{}
		/** Flag to set if the list is read only */
		SLATE_ARGUMENT(bool, ReadOnly)
		/** Comma delimited string of tag root names to filter by */
		SLATE_ARGUMENT(FString, Filter)
		/** Delegate to call when dialog Ok is pressed. */
		SLATE_EVENT(FSimpleDelegate, OnOk)
		/** Delegate to call when dialog Cancel is pressed. */
		SLATE_EVENT(FSimpleDelegate, OnCancel)
		/** Delegate to call when queries are committed. */	
		SLATE_EVENT(FOnQueriesCommitted, OnQueriesCommitted)
	SLATE_END_ARGS()

	/** Construct the actual widget */
	void Construct(const FArguments& InArgs, const TArray<FGameplayTagQuery>& InTagQueries);

	void SetOnOk(FSimpleDelegate InOnOk) { OnOk = InOnOk; }
	void SetOnCancel(FSimpleDelegate InOnCancel) { OnCancel = InOnCancel; }
	void SetOnQueriesCommitted(FOnQueriesCommitted InOnQueriesCommitted) { OnQueriesCommitted = InOnQueriesCommitted; }

	bool IsReadOnly() const { return bReadOnly; }
	const TArray<FGameplayTagQuery>& GetTagQueries() { return TagQueries; }
	const FString& GetFilter() { return Filter; }

	/** Returns true if this widget is currently bound to an SWindow (i.e. it's possible to commit the changes) */
	bool IsBoundToWindow() const;
	
	virtual ~SGameplayTagQueryWidget() override;

	virtual void NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged ) override;

private:

	void OnGetCategoriesMetaFromPropertyHandle(TSharedPtr<IPropertyHandle> PropertyHandle, FString& MetaString) const;

	/** Called when the user clicks the "Save and Close" button */
	FReply OnOkClicked();

	/** Called when the user clicks the "Close Without Saving" button */
	FReply OnCancelClicked();

	/** Controls visibility of the "Save and Close" button */
	EVisibility GetSaveAndCloseButtonVisibility() const;
	/** Controls visibility of the "Close Without Saving" button */
	EVisibility GetCancelButtonVisibility() const;

	/** Converts editable representation to FGameplayTagQuery and calls OnQueriesCommitted. */
	void SaveToTagQuery();

	/* Flag to set if the list is read only*/
	uint32 bReadOnly : 1;

	/** Containers to modify */
	TArray<FGameplayTagQuery> TagQueries;

	/** Called when values are committed. */
	FOnQueriesCommitted OnQueriesCommitted;

	/** Called when "OK" is clicked. */
	FSimpleDelegate OnOk;

	/** Called when "Cancel" is clicked. */
	FSimpleDelegate OnCancel;

	/** Comma delimited string of tag root names to filter by */
	FString Filter;
	
	/** Properties Tab */
	TSharedPtr<IDetailsView> Details;

	static UEditableGameplayTagQuery* CreateEditableQuery(FGameplayTagQuery& Q);
	
	TWeakObjectPtr<UEditableGameplayTagQuery> EditableQuery;
	
	TArray<FGameplayTagQuery> OriginalTagQueries;
};

struct FGameplayTagQueryWindowArgs
{
	/** Title of the query editor window. */
	FText Title;

	/** Flag to set if the list is read only */
	bool bReadOnly = false;

	/** Delegate to call when the query is changed.  */
	SGameplayTagQueryWidget::FOnQueriesCommitted OnQueriesCommitted;

	/** Comma delimited string of tag root names to filter by */
	FString Filter;
	
	/** The array of queries to edit. */
	TArray<FGameplayTagQuery> EditableQueries;

	/** (Optional) Widget to use to position the query window. */
	TSharedPtr<SWidget> AnchorWidget;

	/** (Optional) The underlying edited property. Used for re-assigning the window upon a detail panel refresh. */
	TWeakFieldPtr<FProperty> Property;
};

namespace UE::GameplayTags::Editor
{
	TWeakPtr<SGameplayTagQueryWidget> OpenGameplayTagQueryWindow(const FGameplayTagQueryWindowArgs& Args);
	void CloseGameplayTagQueryWindow(TWeakPtr<SGameplayTagQueryWidget> QueryWidget);
	TWeakPtr<SGameplayTagQueryWidget> TrySyncGameplayTagQueryWidget(const FGameplayTagQueryWindowArgs& Args);
};