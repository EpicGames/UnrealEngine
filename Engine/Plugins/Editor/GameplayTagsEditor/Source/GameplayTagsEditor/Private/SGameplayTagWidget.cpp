// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGameplayTagWidget.h"
#include "AssetRegistry/AssetIdentifier.h"
#include "Misc/ConfigCacheIni.h"
#include "Framework/Views/TableViewMetadata.h"
#include "Widgets/Input/SButton.h"
#include "GameplayTagsManager.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/IToolTip.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SWindow.h"
#include "Misc/MessageDialog.h"
#include "ScopedTransaction.h"
#include "PropertyHandle.h"
#include "Widgets/Input/SSearchBox.h"
#include "GameplayTagsEditorModule.h"
#include "GameplayTagsModule.h"
#include "Framework/Application/SlateApplication.h"
#include "SAddNewGameplayTagWidget.h"
#include "SAddNewGameplayTagSourceWidget.h"
#include "SRenameGameplayTagDialog.h"
#include "Editor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Interfaces/IMainFrameModule.h"
#include "Widgets/Views/STreeView.h"

#define LOCTEXT_NAMESPACE "GameplayTagWidget"

const FString SGameplayTagWidget::SettingsIniSection = TEXT("GameplayTagWidget");

bool SGameplayTagWidget::EnumerateEditableTagContainersFromPropertyHandle(const TSharedRef<IPropertyHandle>& PropHandle, TFunctionRef<bool(const FEditableGameplayTagContainerDatum&)> Callback)
{
	FStructProperty* StructProperty = CastField<FStructProperty>(PropHandle->GetProperty());
	if (StructProperty && StructProperty->Struct->IsChildOf(FGameplayTagContainer::StaticStruct()))
	{
		PropHandle->EnumerateRawData([&Callback](void* RawData, const int32 /*DataIndex*/, const int32 /*NumDatas*/)
		{
			return Callback(FEditableGameplayTagContainerDatum(nullptr, static_cast<FGameplayTagContainer*>(RawData)));
		});
		return true;
	}
	return false;
}

bool SGameplayTagWidget::GetEditableTagContainersFromPropertyHandle(const TSharedRef<IPropertyHandle>& PropHandle, TArray<FEditableGameplayTagContainerDatum>& OutEditableTagContainers)
{
	FStructProperty* StructProperty = CastField<FStructProperty>(PropHandle->GetProperty());
	if (StructProperty && StructProperty->Struct->IsChildOf(FGameplayTagContainer::StaticStruct()))
	{
		OutEditableTagContainers.Reset();
		return EnumerateEditableTagContainersFromPropertyHandle(PropHandle, [&OutEditableTagContainers](const FEditableGameplayTagContainerDatum& EditableTagContainer)
		{
			OutEditableTagContainers.Add(EditableTagContainer);
			return true;
		});
	}
	return false;
}

void SGameplayTagWidget::Construct(const FArguments& InArgs, const TArray<FEditableGameplayTagContainerDatum>& EditableTagContainers)
{
	TagContainers = EditableTagContainers;
	if (InArgs._PropertyHandle.IsValid())
	{
		// If we're backed by a property handle then try and get the tag containers from the property handle
		GetEditableTagContainersFromPropertyHandle(InArgs._PropertyHandle.ToSharedRef(), TagContainers);
	}

	// If we're in management mode, we don't need to have editable tag containers.
	ensure(TagContainers.Num() > 0 || InArgs._GameplayTagUIMode == EGameplayTagUIMode::ManagementMode);

	OnTagChanged = InArgs._OnTagChanged;
	bReadOnly = InArgs._ReadOnly;
	TagContainerName = InArgs._TagContainerName;
	bMultiSelect = InArgs._MultiSelect;
	PropertyHandle = InArgs._PropertyHandle;
	RootFilterString = InArgs._Filter;
	TagFilter = InArgs._OnFilterTag;
	GameplayTagUIMode = InArgs._GameplayTagUIMode;
	bForceHideAddNewTag = InArgs._ForceHideAddNewTag;
	bForceHideAddNewTagSource = InArgs._ForceHideAddNewTagSource;
	bForceHideTagTreeControls = InArgs._ForceHideTagTreeControls;

	bAddTagSectionExpanded = InArgs._NewTagControlsInitiallyExpanded;
	bDelayRefresh = false;
	MaxHeight = InArgs._MaxHeight;

	bRestrictedTags = InArgs._RestrictedTags;

	IGameplayTagsModule::OnGameplayTagTreeChanged.AddSP(this, &SGameplayTagWidget::RefreshOnNextTick);
	UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

	GetFilteredGameplayRootTags(RootFilterString, TagItems);

	if (bRestrictedTags)
	{
		// We only want to show the restricted gameplay tags
		for (int32 Idx = TagItems.Num() - 1; Idx >= 0; --Idx)
		{
			if (!TagItems[Idx]->IsRestrictedGameplayTag())
			{
				TagItems.RemoveAtSwap(Idx);
			}
		}
	}

	if (Manager.OnFilterGameplayTag.IsBound())
	{
		for (int32 Idx = TagItems.Num() - 1; Idx >= 0; --Idx)
		{
			bool DelegateShouldHide = false;
			FGameplayTagSource* Source = Manager.FindTagSource(TagItems[Idx]->GetFirstSourceName());
			Manager.OnFilterGameplayTag.Broadcast(UGameplayTagsManager::FFilterGameplayTagContext(RootFilterString, TagItems[Idx], Source, PropertyHandle), DelegateShouldHide);
			if (DelegateShouldHide)
			{
				TagItems.RemoveAtSwap(Idx);
			}
		}
	}

	// Tag the assets as transactional so they can support undo/redo
	TArray<UObject*> ObjectsToMarkTransactional;
	if (PropertyHandle.IsValid())
	{
		// If we have a property handle use that to find the objects that need to be transactional
		PropertyHandle->GetOuterObjects(ObjectsToMarkTransactional);
	}
	else
	{
		// Otherwise use the owner list
		for (int32 AssetIdx = 0; AssetIdx < TagContainers.Num(); ++AssetIdx)
		{
			ObjectsToMarkTransactional.Add(TagContainers[AssetIdx].TagContainerOwner.Get());
		}
	}

	// Now actually mark the assembled objects
	for (UObject* ObjectToMark : ObjectsToMarkTransactional)
	{
		if (ObjectToMark)
		{
			ObjectToMark->SetFlags(RF_Transactional);
		}
	}

	const FText NewTagText = bRestrictedTags ? LOCTEXT("AddNewRestrictedTag", "Add New Restricted Gameplay Tag") : LOCTEXT("AddNewTag", "Add New Gameplay Tag");

	ChildSlot
	[
		SNew(SBorder)
		.Padding(InArgs._Padding)
		.BorderImage(InArgs._BackgroundBrush)
		[
			SNew(SVerticalBox)

			// Expandable UI controls
			+SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Top)
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew( SCheckBox )
					.IsChecked(this, &SGameplayTagWidget::GetAddTagSectionExpansionState) 
					.OnCheckStateChanged(this, &SGameplayTagWidget::OnAddTagSectionExpansionStateChanged)
					.CheckedImage(FAppStyle::GetBrush("TreeArrow_Expanded"))
					.CheckedHoveredImage(FAppStyle::GetBrush("TreeArrow_Expanded_Hovered"))
					.CheckedPressedImage(FAppStyle::GetBrush("TreeArrow_Expanded"))
					.UncheckedImage(FAppStyle::GetBrush("TreeArrow_Collapsed"))
					.UncheckedHoveredImage(FAppStyle::GetBrush("TreeArrow_Collapsed_Hovered"))
					.UncheckedPressedImage(FAppStyle::GetBrush("TreeArrow_Collapsed"))
					.Visibility( this, &SGameplayTagWidget::DetermineExpandableUIVisibility )
					[
						SNew( STextBlock )
						.Text( NewTagText )
					]
				]
			]

			// Expandable UI for adding tags
			+SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Top)
			.Padding(16.0f, 0.0f)
			[
				SAssignNew( AddNewTagWidget, SAddNewGameplayTagWidget )
				.Visibility(this, &SGameplayTagWidget::DetermineAddNewTagWidgetVisibility)
				.OnGameplayTagAdded(this, &SGameplayTagWidget::OnGameplayTagAdded)
				.NewTagName(InArgs._NewTagName)
				.RestrictedTags(bRestrictedTags)
			]

			// Expandable UI controls
			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Top)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SCheckBox)
					.IsChecked(this, &SGameplayTagWidget::GetAddSourceSectionExpansionState)
					.OnCheckStateChanged(this, &SGameplayTagWidget::OnAddSourceSectionExpansionStateChanged)
					.CheckedImage(FAppStyle::GetBrush("TreeArrow_Expanded"))
					.CheckedHoveredImage(FAppStyle::GetBrush("TreeArrow_Expanded_Hovered"))
					.CheckedPressedImage(FAppStyle::GetBrush("TreeArrow_Expanded"))
					.UncheckedImage(FAppStyle::GetBrush("TreeArrow_Collapsed"))
					.UncheckedHoveredImage(FAppStyle::GetBrush("TreeArrow_Collapsed_Hovered"))
					.UncheckedPressedImage(FAppStyle::GetBrush("TreeArrow_Collapsed"))
					.Visibility(this, &SGameplayTagWidget::DetermineAddNewSourceExpandableUIVisibility)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("AddNewSource", "Add New Tag Source"))
					]
				]
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Top)
			.Padding(16.0f, 0.0f)
			[
				SAssignNew(AddNewTagSourceWidget, SAddNewGameplayTagSourceWidget)
				.Visibility(this, &SGameplayTagWidget::DetermineAddNewSourceWidgetVisibility)
			]

			// Gameplay Tag Tree controls
			+SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Top)
			[
				SNew(SHorizontalBox)
				.Visibility(this, &SGameplayTagWidget::DetermineTagTreeControlsVisibility)

				// Expand All nodes
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.OnClicked(this, &SGameplayTagWidget::OnExpandAllClicked)
					.Text(LOCTEXT("GameplayTagWidget_ExpandAll", "Expand All"))
				]
			
				// Collapse All nodes
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.OnClicked(this, &SGameplayTagWidget::OnCollapseAllClicked)
					.Text(LOCTEXT("GameplayTagWidget_CollapseAll", "Collapse All"))
				]

				// Clear selections
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.IsEnabled(this, &SGameplayTagWidget::CanSelectTags)
					.OnClicked(this, &SGameplayTagWidget::OnClearAllClicked)
					.Text(LOCTEXT("GameplayTagWidget_ClearAll", "Clear All"))
					.Visibility(this, &SGameplayTagWidget::DetermineClearSelectionVisibility)
				]

				// Search
				+SHorizontalBox::Slot()
				.VAlign( VAlign_Center )
				.FillWidth(1.f)
				.Padding(5,1,5,1)
				[
					SAssignNew(SearchTagBox, SSearchBox)
					.HintText(LOCTEXT("GameplayTagWidget_SearchBoxHint", "Search Gameplay Tags"))
					.OnTextChanged( this, &SGameplayTagWidget::OnFilterTextChanged )
				]
			]

			// Gameplay Tags tree
			+SVerticalBox::Slot()
			.MaxHeight(MaxHeight)
			[
				SAssignNew(TagTreeContainerWidget, SBox)
				[
					SAssignNew(TagTreeWidget, STreeView< TSharedPtr<FGameplayTagNode> >)
					.TreeItemsSource(&TagItems)
					.OnGenerateRow(this, &SGameplayTagWidget::OnGenerateRow)
					.OnGetChildren(this, &SGameplayTagWidget::OnGetChildren)
					.OnExpansionChanged( this, &SGameplayTagWidget::OnExpansionChanged)
					.SelectionMode(ESelectionMode::Multi)
				]
			]
		]
	];

	if (InArgs._TagTreeViewBackgroundBrush)
	{
		TagTreeWidget->SetBackgroundBrush(InArgs._TagTreeViewBackgroundBrush);
	}

	// Force the entire tree collapsed to start
	SetTagTreeItemExpansion(false);

	LoadSettings();

	VerifyAssetTagValidity();
}

void SGameplayTagWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (PropertyHandle.IsValid())
	{
		// If we're backed by a property handle then try and refresh the tag containers, 
		// as they may have changed under us (eg, from object re-instancing)
		GetEditableTagContainersFromPropertyHandle(PropertyHandle.ToSharedRef(), TagContainers);
	}

	if (bDelayRefresh)
	{
		RefreshTags();
		bDelayRefresh = false;
	}
}

FVector2D SGameplayTagWidget::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	FVector2D WidgetSize = SCompoundWidget::ComputeDesiredSize(LayoutScaleMultiplier);

	FVector2D TagTreeContainerSize = TagTreeContainerWidget->GetDesiredSize();

	if (TagTreeContainerSize.Y < MaxHeight)
	{
		WidgetSize.Y += MaxHeight - TagTreeContainerSize.Y;
	}

	return WidgetSize;
}

void SGameplayTagWidget::OnFilterTextChanged( const FText& InFilterText )
{
	FilterString = InFilterText.ToString();	

	FilterTagTree();
}

void SGameplayTagWidget::FilterTagTree()
{
	if (FilterString.IsEmpty())
	{
		TagTreeWidget->SetTreeItemsSource(&TagItems);

		for (int32 iItem = 0; iItem < TagItems.Num(); ++iItem)
		{
			SetDefaultTagNodeItemExpansion(TagItems[iItem]);
		}
	}
	else
	{
		FilteredTagItems.Empty();

		for (int32 iItem = 0; iItem < TagItems.Num(); ++iItem)
		{
			if (FilterChildrenCheck(TagItems[iItem]))
			{
				FilteredTagItems.Add(TagItems[iItem]);
				SetTagNodeItemExpansion(TagItems[iItem], true);
			}
			else
			{
				SetTagNodeItemExpansion(TagItems[iItem], false);
			}
		}

		TagTreeWidget->SetTreeItemsSource(&FilteredTagItems);
	}

	TagTreeWidget->RequestTreeRefresh();
}

bool SGameplayTagWidget::FilterChildrenCheck( TSharedPtr<FGameplayTagNode> InItem )
{
	if( !InItem.IsValid() )
	{
		return false;
	}

	if (bRestrictedTags && !InItem->IsRestrictedGameplayTag())
	{
		return false;
	}

	auto FilterChildrenCheck_r = ([this, InItem]()
	{
		TArray< TSharedPtr<FGameplayTagNode> > Children = InItem->GetChildTagNodes();
		for( int32 iChild = 0; iChild < Children.Num(); ++iChild )
		{
			if( FilterChildrenCheck( Children[iChild] ) )
			{
				return true;
			}
		}
		return false;
	});

	UGameplayTagsManager& Manager = UGameplayTagsManager::Get();
	bool DelegateShouldHide = false;
	Manager.OnFilterGameplayTagChildren.Broadcast(RootFilterString, InItem, DelegateShouldHide);
	if (!DelegateShouldHide && Manager.OnFilterGameplayTag.IsBound())
	{
		FGameplayTagSource* Source = Manager.FindTagSource(InItem->GetFirstSourceName());
		Manager.OnFilterGameplayTag.Broadcast(UGameplayTagsManager::FFilterGameplayTagContext(RootFilterString, InItem, Source, PropertyHandle), DelegateShouldHide);
	}
	if (DelegateShouldHide)
	{
		// The delegate wants to hide, see if any children need to show
		return FilterChildrenCheck_r();
	}

	if( InItem->GetCompleteTagString().Contains( FilterString ) || FilterString.IsEmpty() )
	{
		return true;
	}

	return FilterChildrenCheck_r();
}

TSharedRef<ITableRow> SGameplayTagWidget::OnGenerateRow(TSharedPtr<FGameplayTagNode> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	FText TooltipText;
	if (InItem.IsValid())
	{
		UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

		FName TagName = InItem.Get()->GetCompleteTagName();
		TSharedPtr<FGameplayTagNode> Node = Manager.FindTagNode(TagName);

		FString TooltipString = TagName.ToString();

		if (Node.IsValid())
		{
			FName TagSource;
			if (Node->bIsExplicitTag)
			{
				TagSource = Node->GetFirstSourceName();
			}
			else
			{
				TagSource = FName(TEXT("Implicit"));
			}
			TooltipString.Appendf(TEXT(" (%s)"), *TagSource.ToString());

			// tag comments
			if (!Node->DevComment.IsEmpty())
			{
				TooltipString.Appendf(TEXT("\n\n%s"), *Node->DevComment);
			}

			// info related to conflicts
			if (Node->bDescendantHasConflict)
			{
				TooltipString.Append(TEXT("\n\nA tag that descends from this tag has a source conflict."));
			}

			if (Node->bAncestorHasConflict)
			{
				TooltipString.Append(TEXT("\n\nThis tag is descended from a tag that has a conflict. No operations can be performed on this tag until the conflict is resolved."));
			}

			if (Node->bNodeHasConflict)
			{
				TooltipString.Append(TEXT("\n\nThis tag comes from multiple sources. Tags may only have one source."));
			}
		}

		TooltipText = FText::FromString(MoveTemp(TooltipString));
	}

	return SNew(STableRow< TSharedPtr<FGameplayTagNode> >, OwnerTable)
		.Style(FAppStyle::Get(), "GameplayTagTreeView")
		[
			SNew( SHorizontalBox )

			// Tag Selection (selection mode only)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Left)
			[
				SNew(SCheckBox)
				.OnCheckStateChanged(this, &SGameplayTagWidget::OnTagCheckStatusChanged, InItem)
				.IsChecked(this, &SGameplayTagWidget::IsTagChecked, InItem)
				.ToolTipText(TooltipText)
				.IsEnabled(this, &SGameplayTagWidget::CanSelectTags)
				.Visibility( GameplayTagUIMode == EGameplayTagUIMode::SelectionMode || GameplayTagUIMode == EGameplayTagUIMode::HybridMode ? EVisibility::Visible : EVisibility::Collapsed )
				[
					SNew(STextBlock)
					.Text(FText::FromName(InItem->GetSimpleTagName()))
				]
			]

			// Normal Tag Display (management mode only)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Left)
			[
				SNew( STextBlock )
				.ToolTip( FSlateApplication::Get().MakeToolTip(TooltipText) )
				.Text(FText::FromName( InItem->GetSimpleTagName()) )
				.ColorAndOpacity(this, &SGameplayTagWidget::GetTagTextColour, InItem)
				.Visibility( GameplayTagUIMode == EGameplayTagUIMode::ManagementMode ? EVisibility::Visible : EVisibility::Collapsed )
			]

			// Allows non-restricted children checkbox
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			[
				SNew(SCheckBox)
				.ToolTipText(LOCTEXT("AllowsChildren", "Does this restricted tag allow non-restricted children"))
				.OnCheckStateChanged(this, &SGameplayTagWidget::OnAllowChildrenTagCheckStatusChanged, InItem)
				.IsChecked(this, &SGameplayTagWidget::IsAllowChildrenTagChecked, InItem)
				.Visibility(this, &SGameplayTagWidget::DetermineAllowChildrenVisible, InItem)
			]

			// Add Subtag
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			[
				SNew( SButton )
				.ToolTipText( LOCTEXT("AddSubtag", "Add Subtag") )
				.Visibility(this, &SGameplayTagWidget::DetermineAddNewSubTagWidgetVisibility, InItem)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.OnClicked( this, &SGameplayTagWidget::OnAddSubtagClicked, InItem )
				.DesiredSizeScale(FVector2D(0.75f, 0.75f))
				.ContentPadding(4.0f)
				.ForegroundColor(FSlateColor::UseForeground())
				.IsEnabled( !bReadOnly )
				.IsFocusable( false )
				[
					SNew( SImage )
					.Image(FAppStyle::GetBrush("Icons.PlusCircle"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]

			// More Actions Menu
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			[
				SNew( SComboButton )
				.ToolTipText( LOCTEXT("MoreActions", "More Actions...") )
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.ContentPadding(0)
				.ForegroundColor(FSlateColor::UseForeground())
				.HasDownArrow(true)
				.OnGetMenuContent(this, &SGameplayTagWidget::MakeTagActionsMenu, InItem)
				.CollapseMenuOnParentFocus(true)
			]
		];
}

void SGameplayTagWidget::OnGetChildren(TSharedPtr<FGameplayTagNode> InItem, TArray< TSharedPtr<FGameplayTagNode> >& OutChildren)
{
	TArray< TSharedPtr<FGameplayTagNode> > FilteredChildren;
	TArray< TSharedPtr<FGameplayTagNode> > Children = InItem->GetChildTagNodes();

	for( int32 iChild = 0; iChild < Children.Num(); ++iChild )
	{
		if( FilterChildrenCheck( Children[iChild] ) )
		{
			FilteredChildren.Add( Children[iChild] );
		}
	}
	OutChildren += FilteredChildren;
}

void SGameplayTagWidget::OnTagCheckStatusChanged(ECheckBoxState NewCheckState, TSharedPtr<FGameplayTagNode> NodeChanged)
{
	if (NewCheckState == ECheckBoxState::Checked)
	{
		OnTagChecked(NodeChanged);
	}
	else if (NewCheckState == ECheckBoxState::Unchecked)
	{
		OnTagUnchecked(NodeChanged);
	}
}

void SGameplayTagWidget::OnTagChecked(TSharedPtr<FGameplayTagNode> NodeChecked)
{
	FScopedTransaction Transaction( LOCTEXT("GameplayTagWidget_AddTags", "Add Gameplay Tags") );

	for (int32 ContainerIdx = 0; ContainerIdx < TagContainers.Num(); ++ContainerIdx)
	{
		TSharedPtr<FGameplayTagNode> CurNode(NodeChecked);
		UObject* OwnerObj = TagContainers[ContainerIdx].TagContainerOwner.Get();
		FGameplayTagContainer* Container = TagContainers[ContainerIdx].TagContainer;

		if (Container)
		{
			FGameplayTagContainer EditableContainer = *Container;

			bool bRemoveParents = false;

			while (CurNode.IsValid())
			{
				FGameplayTag GameplayTag = CurNode->GetCompleteTag();

				if (bRemoveParents == false)
				{
					bRemoveParents = true;
					if (bMultiSelect == false)
					{
						EditableContainer.Reset();
					}
					EditableContainer.AddTag(GameplayTag);
				}
				else
				{
					EditableContainer.RemoveTag(GameplayTag);
				}

				CurNode = CurNode->GetParentTagNode();
			}
			SetContainer(Container, &EditableContainer, OwnerObj);
		}
	}
}

void SGameplayTagWidget::OnTagUnchecked(TSharedPtr<FGameplayTagNode> NodeUnchecked)
{
	FScopedTransaction Transaction( LOCTEXT("GameplayTagWidget_RemoveTags", "Remove Gameplay Tags"));
	if (NodeUnchecked.IsValid())
	{
		UGameplayTagsManager& TagsManager = UGameplayTagsManager::Get();

		for (int32 ContainerIdx = 0; ContainerIdx < TagContainers.Num(); ++ContainerIdx)
		{
			UObject* OwnerObj = TagContainers[ContainerIdx].TagContainerOwner.Get();
			FGameplayTagContainer* Container = TagContainers[ContainerIdx].TagContainer;
			FGameplayTag GameplayTag = NodeUnchecked->GetCompleteTag();

			if (Container)
			{
				FGameplayTagContainer EditableContainer = *Container;
				EditableContainer.RemoveTag(GameplayTag);

				TSharedPtr<FGameplayTagNode> ParentNode = NodeUnchecked->GetParentTagNode();
				if (ParentNode.IsValid())
				{
					// Check if there are other siblings before adding parent
					bool bOtherSiblings = false;
					for (auto It = ParentNode->GetChildTagNodes().CreateConstIterator(); It; ++It)
					{
						GameplayTag = It->Get()->GetCompleteTag();
						if (EditableContainer.HasTagExact(GameplayTag))
						{
							bOtherSiblings = true;
							break;
						}
					}
					// Add Parent
					if (!bOtherSiblings)
					{
						GameplayTag = ParentNode->GetCompleteTag();
						EditableContainer.AddTag(GameplayTag);
					}
				}

				// Uncheck Children
				for (const auto& ChildNode : NodeUnchecked->GetChildTagNodes())
				{
					UncheckChildren(ChildNode, EditableContainer);
				}

				SetContainer(Container, &EditableContainer, OwnerObj);
			}
			
		}
	}
}

void SGameplayTagWidget::UncheckChildren(TSharedPtr<FGameplayTagNode> NodeUnchecked, FGameplayTagContainer& EditableContainer)
{
	UGameplayTagsManager& TagsManager = UGameplayTagsManager::Get();

	FGameplayTag GameplayTag = NodeUnchecked->GetCompleteTag();
	EditableContainer.RemoveTag(GameplayTag);

	// Uncheck Children
	for (const auto& ChildNode : NodeUnchecked->GetChildTagNodes())
	{
		UncheckChildren(ChildNode, EditableContainer);
	}
}

ECheckBoxState SGameplayTagWidget::IsTagChecked(TSharedPtr<FGameplayTagNode> Node) const
{
	int32 NumValidAssets = 0;
	int32 NumAssetsTagIsAppliedTo = 0;

	if (Node.IsValid())
	{
		UGameplayTagsManager& TagsManager = UGameplayTagsManager::Get();

		for (int32 ContainerIdx = 0; ContainerIdx < TagContainers.Num(); ++ContainerIdx)
		{
			FGameplayTagContainer* Container = TagContainers[ContainerIdx].TagContainer;
			if (Container)
			{
				NumValidAssets++;
				FGameplayTag GameplayTag = Node->GetCompleteTag();
				if (GameplayTag.IsValid())
				{
					if (Container->HasTag(GameplayTag))
					{
						++NumAssetsTagIsAppliedTo;
					}
				}
			}
		}
	}

	if (NumAssetsTagIsAppliedTo == 0)
	{
		return ECheckBoxState::Unchecked;
	}
	else if (NumAssetsTagIsAppliedTo == NumValidAssets)
	{
		return ECheckBoxState::Checked;
	}
	else
	{
		return ECheckBoxState::Undetermined;
	}
}

bool SGameplayTagWidget::IsExactTagInCollection(TSharedPtr<FGameplayTagNode> Node) const
{
	if (Node.IsValid())
	{
		UGameplayTagsManager& TagsManager = UGameplayTagsManager::Get();

		for (int32 ContainerIdx = 0; ContainerIdx < TagContainers.Num(); ++ContainerIdx)
		{
			FGameplayTagContainer* Container = TagContainers[ContainerIdx].TagContainer;
			if (Container)
			{
				FGameplayTag GameplayTag = Node->GetCompleteTag();
				if (GameplayTag.IsValid())
				{
					if (Container->HasTagExact(GameplayTag))
					{
						return true;
					}
				}
			}
		}
	}

	return false;
}

void SGameplayTagWidget::OnAllowChildrenTagCheckStatusChanged(ECheckBoxState NewCheckState, TSharedPtr<FGameplayTagNode> NodeChanged)
{
	IGameplayTagsEditorModule& TagsEditor = IGameplayTagsEditorModule::Get();

	if (TagsEditor.UpdateTagInINI(NodeChanged->GetCompleteTagString(), NodeChanged->DevComment, NodeChanged->bIsRestrictedTag, NewCheckState == ECheckBoxState::Checked))
	{
		if (NewCheckState == ECheckBoxState::Checked)
		{
			NodeChanged->bAllowNonRestrictedChildren = true;
		}
		else if (NewCheckState == ECheckBoxState::Unchecked)
		{
			NodeChanged->bAllowNonRestrictedChildren = false;
		}
	}
}

ECheckBoxState SGameplayTagWidget::IsAllowChildrenTagChecked(TSharedPtr<FGameplayTagNode> Node) const
{
	if (Node->GetAllowNonRestrictedChildren())
	{
		return ECheckBoxState::Checked;
	}

	return ECheckBoxState::Unchecked;
}

EVisibility SGameplayTagWidget::DetermineAllowChildrenVisible(TSharedPtr<FGameplayTagNode> Node) const
{
	// We do not allow you to modify nodes that have a conflict or inherit from a node with a conflict
	if (Node->bNodeHasConflict || Node->bAncestorHasConflict)
	{
		return EVisibility::Hidden;
	}

	if (bRestrictedTags)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

FReply SGameplayTagWidget::OnClearAllClicked()
{
	FScopedTransaction Transaction( LOCTEXT("GameplayTagWidget_RemoveAllTags", "Remove All Gameplay Tags") );

	for (int32 ContainerIdx = 0; ContainerIdx < TagContainers.Num(); ++ContainerIdx)
	{
		UObject* OwnerObj = TagContainers[ContainerIdx].TagContainerOwner.Get();
		FGameplayTagContainer* Container = TagContainers[ContainerIdx].TagContainer;

		if (Container)
		{
			FGameplayTagContainer EmptyContainer;
			SetContainer(Container, &EmptyContainer, OwnerObj);
		}
	}
	return FReply::Handled();
}

FSlateColor SGameplayTagWidget::GetTagTextColour(TSharedPtr<FGameplayTagNode> Node) const
{
	static const FLinearColor DefaultTextColour = FLinearColor::White;
	static const FLinearColor DescendantConflictTextColour = FLinearColor(1.f, 0.65f, 0.f); // orange
	static const FLinearColor NodeConflictTextColour = FLinearColor::Red;
	static const FLinearColor AncestorConflictTextColour = FLinearColor(1.f, 1.f, 1.f, 0.5f);

	if (Node->bNodeHasConflict)
	{
		return NodeConflictTextColour;
	}

	if (Node->bDescendantHasConflict)
	{
		return DescendantConflictTextColour;
	}

	if (Node->bAncestorHasConflict)
	{
		return AncestorConflictTextColour;
	}

	return DefaultTextColour;
}

FReply SGameplayTagWidget::OnExpandAllClicked()
{
	SetTagTreeItemExpansion(true);
	return FReply::Handled();
}

FReply SGameplayTagWidget::OnCollapseAllClicked()
{
	SetTagTreeItemExpansion(false);
	return FReply::Handled();
}

FReply SGameplayTagWidget::OnAddSubtagClicked(TSharedPtr<FGameplayTagNode> InTagNode)
{
	if (!bReadOnly && InTagNode.IsValid())
	{
		UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

		FString TagName = InTagNode->GetCompleteTagString();
		FString TagComment;
		FName TagSource;
		bool bTagIsExplicit;
		bool bTagIsRestricted;
		bool bTagAllowsNonRestrictedChildren;

		Manager.GetTagEditorData(InTagNode->GetCompleteTagName(), TagComment, TagSource, bTagIsExplicit, bTagIsRestricted, bTagAllowsNonRestrictedChildren);

		if (AddNewTagWidget.IsValid())
		{
			bAddTagSectionExpanded = true; 
			AddNewTagWidget->AddSubtagFromParent(TagName, TagSource);
		}
	}
	return FReply::Handled();
}

TSharedRef<SWidget> SGameplayTagWidget::MakeTagActionsMenu(TSharedPtr<FGameplayTagNode> InTagNode)
{
	bool bShowManagement = ((GameplayTagUIMode == EGameplayTagUIMode::ManagementMode || GameplayTagUIMode == EGameplayTagUIMode::HybridMode) && !bReadOnly);
	UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

	if (!Manager.ShouldImportTagsFromINI())
	{
		bShowManagement = false;
	}

	// You can't modify restricted tags in the normal tag menus
	if (!bRestrictedTags && InTagNode->IsRestrictedGameplayTag())
	{
		bShowManagement = false;
	}

	// we can only rename or delete tags if they came from an ini file
	if (!InTagNode->GetFirstSourceName().ToString().EndsWith(TEXT(".ini")))
	{
		bShowManagement = false;
	}

	// Do not close menu after selection. The close deletes this widget before action is executed leading to no action being performed.
	// Occurs when SGameplayTagWidget is being used as a menu item itself (Details panel of blueprint editor for example).
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/ false, nullptr);

	// Rename
	if (bShowManagement)
	{
		FExecuteAction RenameAction = FExecuteAction::CreateSP(this, &SGameplayTagWidget::OnRenameTag, InTagNode);

		MenuBuilder.AddMenuEntry(LOCTEXT("GameplayTagWidget_RenameTag", "Rename"), LOCTEXT("GameplayTagWidget_RenameTagTooltip", "Rename this tag"), FSlateIcon(), FUIAction(RenameAction));
	}

	// Delete
	if (bShowManagement)
	{
		FExecuteAction DeleteAction = FExecuteAction::CreateSP(this, &SGameplayTagWidget::OnDeleteTag, InTagNode);

		MenuBuilder.AddMenuEntry(LOCTEXT("GameplayTagWidget_DeleteTag", "Delete"), LOCTEXT("GameplayTagWidget_DeleteTagTooltip", "Delete this tag"), FSlateIcon(), FUIAction(DeleteAction));
	}

	// Only include these menu items if we have tag containers to modify
	if (TagContainers.Num() > 0 && !bForceHideAddNewTag)
	{
		// Either Remove or Add Exact Tag depending on if we have the exact tag or not
		if (IsExactTagInCollection(InTagNode))
		{
			FExecuteAction RemoveAction = FExecuteAction::CreateSP(this, &SGameplayTagWidget::OnRemoveTag, InTagNode);
			MenuBuilder.AddMenuEntry(LOCTEXT("GameplayTagWidget_RemoveTag", "Remove Exact Tag"), LOCTEXT("GameplayTagWidget_RemoveTagTooltip", "Remove this exact tag, Parent and Child Tags will not be effected."), FSlateIcon(), FUIAction(RemoveAction));
		}
		else
		{
			FExecuteAction AddAction = FExecuteAction::CreateSP(this, &SGameplayTagWidget::OnAddTag, InTagNode);
			MenuBuilder.AddMenuEntry(LOCTEXT("GameplayTagWidget_AddTag", "Add Exact Tag"), LOCTEXT("GameplayTagWidget_AddTagTooltip", "Add this exact tag, Parent and Child Child Tags will not be effected."), FSlateIcon(), FUIAction(AddAction));
		}
	}

	// Search for References
	if (FEditorDelegates::OnOpenReferenceViewer.IsBound())
	{
		FExecuteAction SearchForReferencesAction = FExecuteAction::CreateSP(this, &SGameplayTagWidget::OnSearchForReferences, InTagNode);
		MenuBuilder.AddMenuEntry(LOCTEXT("GameplayTagWidget_SearchForReferences", "Search For References"), LOCTEXT("GameplayTagWidget_SearchForReferencesTooltip", "Find references for this tag"), FSlateIcon(), FUIAction(SearchForReferencesAction));
	}

	return MenuBuilder.MakeWidget();
}

void SGameplayTagWidget::OnRenameTag(TSharedPtr<FGameplayTagNode> InTagNode)
{
	if (InTagNode.IsValid())
	{
		OpenRenameGameplayTagDialog( InTagNode );
	}
}

void SGameplayTagWidget::OnDeleteTag(TSharedPtr<FGameplayTagNode> InTagNode)
{
	if (InTagNode.IsValid())
	{
		IGameplayTagsEditorModule& TagsEditor = IGameplayTagsEditorModule::Get();
		
		bool TagRemoved = false;
		
		if (GameplayTagUIMode == EGameplayTagUIMode::HybridMode)
		{
			for (int32 ContainerIdx = 0; ContainerIdx < TagContainers.Num(); ++ContainerIdx)
			{
				FGameplayTagContainer* Container = TagContainers[ContainerIdx].TagContainer;
				TagRemoved |= Container->RemoveTag(InTagNode->GetCompleteTag());
			}
		}

		const bool bDeleted = TagsEditor.DeleteTagFromINI(InTagNode);

		if (bDeleted || TagRemoved)
		{
			OnTagChanged.ExecuteIfBound();
		}
	}
}

void SGameplayTagWidget::OnAddTag(TSharedPtr<FGameplayTagNode> InTagNode)
{
	if (InTagNode.IsValid())
	{
		for (int32 ContainerIdx = 0; ContainerIdx < TagContainers.Num(); ++ContainerIdx)
		{
			FGameplayTagContainer* Container = TagContainers[ContainerIdx].TagContainer;
			Container->AddTag(InTagNode->GetCompleteTag());
		}

		OnTagChanged.ExecuteIfBound();
	}
}

void SGameplayTagWidget::OnRemoveTag(TSharedPtr<FGameplayTagNode> InTagNode)
{
	if (InTagNode.IsValid())
	{
		for (int32 ContainerIdx = 0; ContainerIdx < TagContainers.Num(); ++ContainerIdx)
		{
			FGameplayTagContainer* Container = TagContainers[ContainerIdx].TagContainer;
			Container->RemoveTag(InTagNode->GetCompleteTag());
		}

		OnTagChanged.ExecuteIfBound();
	}
}

void SGameplayTagWidget::OnSearchForReferences(TSharedPtr<FGameplayTagNode> InTagNode)
{
	if (InTagNode.IsValid())
	{
		TArray<FAssetIdentifier> AssetIdentifiers;
		AssetIdentifiers.Add(FAssetIdentifier(FGameplayTag::StaticStruct(), InTagNode->GetCompleteTagName()));
		FEditorDelegates::OnOpenReferenceViewer.Broadcast(AssetIdentifiers, FReferenceViewerParams());
	}
}

void SGameplayTagWidget::SetTagTreeItemExpansion(bool bExpand)
{
	TArray< TSharedPtr<FGameplayTagNode> > TagArray;
	GetFilteredGameplayRootTags(TEXT(""), TagArray);
	for (int32 TagIdx = 0; TagIdx < TagArray.Num(); ++TagIdx)
	{
		SetTagNodeItemExpansion(TagArray[TagIdx], bExpand);
	}
	
}

void SGameplayTagWidget::SetTagNodeItemExpansion(TSharedPtr<FGameplayTagNode> Node, bool bExpand)
{
	if (Node.IsValid() && TagTreeWidget.IsValid())
	{
		TagTreeWidget->SetItemExpansion(Node, bExpand);

		const TArray< TSharedPtr<FGameplayTagNode> >& ChildTags = Node->GetChildTagNodes();
		for (int32 ChildIdx = 0; ChildIdx < ChildTags.Num(); ++ChildIdx)
		{
			SetTagNodeItemExpansion(ChildTags[ChildIdx], bExpand);
		}
	}
}

void SGameplayTagWidget::LoadSettings()
{
	MigrateSettings();

	TArray< TSharedPtr<FGameplayTagNode> > TagArray;
	GetFilteredGameplayRootTags(TEXT(""), TagArray);
	for (int32 TagIdx = 0; TagIdx < TagArray.Num(); ++TagIdx)
	{
		LoadTagNodeItemExpansion(TagArray[TagIdx] );
	}
}

const FString& SGameplayTagWidget::GetGameplayTagsEditorStateIni()
{
	static FString Filename;

	if (Filename.Len() == 0)
	{
		Filename = FConfigCacheIni::NormalizeConfigIniPath(FString::Printf(TEXT("%s%s/GameplayTagsEditorState.ini"), *FPaths::GeneratedConfigDir(), ANSI_TO_TCHAR(FPlatformProperties::PlatformName())));
	}

	return Filename;
}

void SGameplayTagWidget::MigrateSettings()
{
	if (const FConfigSection* EditorPerProjectIniSection = GConfig->GetSection(*SettingsIniSection, /*Force=*/false, GEditorPerProjectIni))
	{
		if (EditorPerProjectIniSection->Num() > 0)
		{
			const FString& DestFilename = GetGameplayTagsEditorStateIni();
			for (const auto& It : *EditorPerProjectIniSection)
			{
				GConfig->AddUniqueToSection(*SettingsIniSection, It.Key, It.Value.GetSavedValue(), DestFilename);
			}

			GConfig->Flush(false, DestFilename);
		}

		GConfig->EmptySection(*SettingsIniSection, GEditorPerProjectIni);
		GConfig->Flush(false, GEditorPerProjectIni);
	}
}

void SGameplayTagWidget::SetDefaultTagNodeItemExpansion( TSharedPtr<FGameplayTagNode> Node )
{
	if ( Node.IsValid() && TagTreeWidget.IsValid() )
	{
		bool bExpanded = false;

		if ( IsTagChecked(Node) == ECheckBoxState::Checked )
		{
			bExpanded = true;
		}
		TagTreeWidget->SetItemExpansion(Node, bExpanded);

		const TArray< TSharedPtr<FGameplayTagNode> >& ChildTags = Node->GetChildTagNodes();
		for ( int32 ChildIdx = 0; ChildIdx < ChildTags.Num(); ++ChildIdx )
		{
			SetDefaultTagNodeItemExpansion(ChildTags[ChildIdx]);
		}
	}
}

void SGameplayTagWidget::LoadTagNodeItemExpansion( TSharedPtr<FGameplayTagNode> Node )
{
	if (Node.IsValid() && TagTreeWidget.IsValid())
	{
		bool bExpanded = false;

		if( GConfig->GetBool(*SettingsIniSection, *(TagContainerName + Node->GetCompleteTagString() + TEXT(".Expanded")), bExpanded, GetGameplayTagsEditorStateIni()) )
		{
			TagTreeWidget->SetItemExpansion( Node, bExpanded );
		}
		else if( IsTagChecked( Node ) == ECheckBoxState::Checked ) // If we have no save data but its ticked then we probably lost our settings so we shall expand it
		{
			TagTreeWidget->SetItemExpansion( Node, true );
		}

		const TArray< TSharedPtr<FGameplayTagNode> >& ChildTags = Node->GetChildTagNodes();
		for (int32 ChildIdx = 0; ChildIdx < ChildTags.Num(); ++ChildIdx)
		{
			LoadTagNodeItemExpansion(ChildTags[ChildIdx]);
		}
	}
}

void SGameplayTagWidget::OnExpansionChanged( TSharedPtr<FGameplayTagNode> InItem, bool bIsExpanded )
{
	// Save the new expansion setting to ini file
	GConfig->SetBool(*SettingsIniSection, *(TagContainerName + InItem->GetCompleteTagString() + TEXT(".Expanded")), bIsExpanded, GetGameplayTagsEditorStateIni());
}

void SGameplayTagWidget::SetContainer(FGameplayTagContainer* OriginalContainer, FGameplayTagContainer* EditedContainer, UObject* OwnerObj)
{
	if (PropertyHandle.IsValid() && bMultiSelect)
	{
		// Case for a tag container 
		PropertyHandle->SetValueFromFormattedString(EditedContainer->ToString());
	}
	else if (PropertyHandle.IsValid() && !bMultiSelect)
	{
		// Case for a single Tag		
		FString FormattedString = TEXT("(TagName=\"");
		FormattedString += EditedContainer->First().GetTagName().ToString();
		FormattedString += TEXT("\")");
		PropertyHandle->SetValueFromFormattedString(FormattedString);
	}
	else
	{
		// Not sure if we should get here, means the property handle hasn't been setup which could be right or wrong.
		if (OwnerObj)
		{
			OwnerObj->PreEditChange(PropertyHandle.IsValid() ? PropertyHandle->GetProperty() : nullptr);
		}

		*OriginalContainer = *EditedContainer;

		if (OwnerObj)
		{
			OwnerObj->PostEditChange();
		}
	}	

	if (!PropertyHandle.IsValid())
	{
		OnTagChanged.ExecuteIfBound();
	}
}

void SGameplayTagWidget::OnGameplayTagAdded(const FString& TagName, const FString& TagComment, const FName& TagSource)
{
	UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

	RefreshTags();
	TagTreeWidget->RequestTreeRefresh();

	if (GameplayTagUIMode == EGameplayTagUIMode::SelectionMode)
	{
		TSharedPtr<FGameplayTagNode> TagNode = Manager.FindTagNode(FName(*TagName));
		if (TagNode.IsValid())
		{
			OnTagChecked(TagNode);
		}

		// Filter on the new tag
		SearchTagBox->SetText(FText::FromString(TagName));

		// Close the Add New Tag UI
		bAddTagSectionExpanded = false;
	}
}

void SGameplayTagWidget::RefreshTags()
{
	UGameplayTagsManager& Manager = UGameplayTagsManager::Get();
	GetFilteredGameplayRootTags(RootFilterString, TagItems);

	if (bRestrictedTags)
	{
		// We only want to show the restricted gameplay tags
		for (int32 Idx = TagItems.Num() - 1; Idx >= 0; --Idx)
		{
			if (!TagItems[Idx]->IsRestrictedGameplayTag())
			{
				TagItems.RemoveAtSwap(Idx);
			}
		}
	}

	if (Manager.OnFilterGameplayTag.IsBound())
	{
		for (int32 Idx = TagItems.Num() - 1; Idx >= 0; --Idx)
		{
			bool DelegateShouldHide = false;
			FGameplayTagSource* Source = Manager.FindTagSource(TagItems[Idx]->GetFirstSourceName());
			Manager.OnFilterGameplayTag.Broadcast(UGameplayTagsManager::FFilterGameplayTagContext(RootFilterString, TagItems[Idx], Source, PropertyHandle), DelegateShouldHide);
			if (DelegateShouldHide)
			{
				TagItems.RemoveAtSwap(Idx);
			}
		}
	}

	FilterTagTree();
}

EVisibility SGameplayTagWidget::DetermineExpandableUIVisibility() const
{
	UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

	if ( !Manager.ShouldImportTagsFromINI() || (bForceHideAddNewTag && bForceHideAddNewTagSource))
	{
		// If we can't support adding tags from INI files, or both options are forcibly disabled, we should never see this widget
		return EVisibility::Collapsed;
	}

	return EVisibility::Visible;
}

EVisibility SGameplayTagWidget::DetermineAddNewSourceExpandableUIVisibility() const
{
	if (bRestrictedTags || bForceHideAddNewTagSource)
	{
		return EVisibility::Collapsed;
	}

	return DetermineExpandableUIVisibility();
}

EVisibility SGameplayTagWidget::DetermineAddNewTagWidgetVisibility() const
{
	UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

	if ( !Manager.ShouldImportTagsFromINI() || !bAddTagSectionExpanded || bRestrictedTags || bForceHideAddNewTag )
	{
		// If we can't support adding tags from INI files, we should never see this widget
		return EVisibility::Collapsed;
	}

	return EVisibility::Visible;
}

EVisibility SGameplayTagWidget::DetermineAddNewSourceWidgetVisibility() const
{
	UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

	if (!Manager.ShouldImportTagsFromINI() || !bAddSourceSectionExpanded || bRestrictedTags || bForceHideAddNewTagSource )
	{
		// If we can't support adding tags from INI files, we should never see this widget
		return EVisibility::Collapsed;
	}

	return EVisibility::Visible;
}

EVisibility SGameplayTagWidget::DetermineTagTreeControlsVisibility() const
{
	if (bForceHideTagTreeControls)
	{
		return EVisibility::Collapsed;
	}

	return EVisibility::Visible;
}

EVisibility SGameplayTagWidget::DetermineAddNewSubTagWidgetVisibility(TSharedPtr<FGameplayTagNode> Node) const
{
	EVisibility LocalVisibility = DetermineExpandableUIVisibility();
	if (LocalVisibility != EVisibility::Visible)
	{
		return LocalVisibility;
	}

	// We do not allow you to add child tags under a conflict
	if (Node->bNodeHasConflict || Node->bAncestorHasConflict)
	{
		return EVisibility::Hidden;
	}

	// show if we're dealing with restricted tags exclusively or restricted tags that allow non-restricted children
	if (Node->GetAllowNonRestrictedChildren() || bRestrictedTags)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Hidden;
}

EVisibility SGameplayTagWidget::DetermineClearSelectionVisibility() const
{
	return CanSelectTags() ? EVisibility::Visible : EVisibility::Collapsed;
}

bool SGameplayTagWidget::CanSelectTags() const
{
	return !bReadOnly && (GameplayTagUIMode == EGameplayTagUIMode::SelectionMode || GameplayTagUIMode == EGameplayTagUIMode::HybridMode);
}

bool SGameplayTagWidget::IsAddingNewTag() const
{
	return AddNewTagWidget.IsValid() && AddNewTagWidget->IsAddingNewTag();
}

ECheckBoxState SGameplayTagWidget::GetAddTagSectionExpansionState() const
{
	return bAddTagSectionExpanded ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SGameplayTagWidget::OnAddTagSectionExpansionStateChanged(ECheckBoxState NewState)
{
	bAddTagSectionExpanded = NewState == ECheckBoxState::Checked;
}

ECheckBoxState SGameplayTagWidget::GetAddSourceSectionExpansionState() const
{
	return bAddSourceSectionExpanded ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SGameplayTagWidget::OnAddSourceSectionExpansionStateChanged(ECheckBoxState NewState)
{
	bAddSourceSectionExpanded = NewState == ECheckBoxState::Checked;
}

void SGameplayTagWidget::RefreshOnNextTick()
{
	bDelayRefresh = true;
}

void SGameplayTagWidget::OnGameplayTagRenamed(FString OldTagName, FString NewTagName)
{
	OnTagChanged.ExecuteIfBound();
}

void SGameplayTagWidget::OpenRenameGameplayTagDialog(TSharedPtr<FGameplayTagNode> GameplayTagNode) const
{
	TSharedRef<SWindow> RenameTagWindow =
		SNew(SWindow)
		.Title(LOCTEXT("RenameTagWindowTitle", "Rename Gameplay Tag"))
		.ClientSize(FVector2D(320.0f, 110.0f))
		.SizingRule(ESizingRule::FixedSize)
		.SupportsMaximize(false)
		.SupportsMinimize(false);

	TSharedRef<SRenameGameplayTagDialog> RenameTagDialog =
		SNew(SRenameGameplayTagDialog)
		.GameplayTagNode(GameplayTagNode)
		.OnGameplayTagRenamed(const_cast<SGameplayTagWidget*>(this), &SGameplayTagWidget::OnGameplayTagRenamed);

	RenameTagWindow->SetContent(RenameTagDialog);

	const TSharedPtr<SWindow> CurrentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
	
	if (CurrentWindow->IsRegularWindow())
	{
		FSlateApplication::Get().AddModalWindow(RenameTagWindow, CurrentWindow);
	}
	else
	{
		// AddModalWindow will crash if the parent window is a pop up, so we parent it to the MainFrame window instead.
		const IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
		FSlateApplication::Get().AddModalWindow(RenameTagWindow, MainFrame.GetParentWindow());
	}
}

TSharedPtr<SWidget> SGameplayTagWidget::GetWidgetToFocusOnOpen()
{
	return SearchTagBox;
}

void SGameplayTagWidget::VerifyAssetTagValidity()
{
	UGameplayTagsManager& TagsManager = UGameplayTagsManager::Get();

	// Find and remove any tags on the asset that are no longer in the library
	for (int32 ContainerIdx = 0; ContainerIdx < TagContainers.Num(); ++ContainerIdx)
	{
		UObject* OwnerObj = TagContainers[ContainerIdx].TagContainerOwner.Get();
		FGameplayTagContainer* Container = TagContainers[ContainerIdx].TagContainer;

		if (Container)
		{
			FGameplayTagContainer EditableContainer = *Container;

			// Use a set instead of a container so we can find and remove None tags
			TSet<FGameplayTag> InvalidTags;

			for (auto It = Container->CreateConstIterator(); It; ++It)
			{
				FGameplayTag TagToCheck = *It;

				if (!UGameplayTagsManager::Get().RequestGameplayTag(TagToCheck.GetTagName(), false).IsValid())
				{
					InvalidTags.Add(*It);
				}
			}

			if (InvalidTags.Num() > 0)
			{
				FString InvalidTagNames;

				for (auto InvalidIter = InvalidTags.CreateConstIterator(); InvalidIter; ++InvalidIter)
				{
					EditableContainer.RemoveTag(*InvalidIter);
					InvalidTagNames += InvalidIter->ToString() + TEXT("\n");
				}
				SetContainer(Container, &EditableContainer, OwnerObj);

				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("Objects"), FText::FromString(InvalidTagNames));
				FText DialogText = FText::Format(LOCTEXT("GameplayTagWidget_InvalidTags", "Invalid Tags that have been removed: \n\n{Objects}"), Arguments);
				FText DialogTitle = LOCTEXT("GameplayTagWidget_Warning", "Warning");
				FMessageDialog::Open(EAppMsgType::Ok, DialogText, DialogTitle);
			}
		}
	}
}

void SGameplayTagWidget::GetFilteredGameplayRootTags(const FString& InFilterString, TArray<TSharedPtr<FGameplayTagNode>>& OutNodes) const
{
	OutNodes.Empty();
	const UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

	if (TagFilter.IsBound())
	{
		TArray<TSharedPtr<FGameplayTagNode>> UnfilteredItems;
		Manager.GetFilteredGameplayRootTags(InFilterString, UnfilteredItems);
		for (const TSharedPtr<FGameplayTagNode>& Node : UnfilteredItems)
		{
			if (TagFilter.Execute(Node) == ETagFilterResult::IncludeTag)
			{
				OutNodes.Add(Node);
			}
		}
	}
	else
	{
		Manager.GetFilteredGameplayRootTags(InFilterString, OutNodes);
	}
}

#undef LOCTEXT_NAMESPACE
