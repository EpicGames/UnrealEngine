// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGraphNode_MultiCompareGameplayTag.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "GraphEditorSettings.h"
#include "GameplayTagsK2Node_MultiCompareBase.h"
#include "ScopedTransaction.h"
#include "Widgets/SToolTip.h"
#include "IDocumentation.h"
#include "Widgets/Text/STextBlock.h"

void SGraphNode_MultiCompareGameplayTag::Construct(const FArguments& InArgs, UGameplayTagsK2Node_MultiCompareBase* InNode)
{
	GraphNode = InNode;

	SetCursor(EMouseCursor::CardinalCross);

	UpdateGraphNode();
	CreateOutputSideRemoveButton(RightNodeBox);
}

void SGraphNode_MultiCompareGameplayTag::CreateOutputSideRemoveButton(TSharedPtr<SVerticalBox> OutputBox)
{
	TSharedPtr<SWidget> ButtonContent;
	SAssignNew(ButtonContent, SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		[
			SNew(STextBlock)
			.Text(NSLOCTEXT("CompareNode", "CompareNodeRemovePinButton", "Remove Case"))
			.ColorAndOpacity(FLinearColor::White)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(7, 0, 0, 0)
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush(TEXT("Icons.Minus")))
		];

	TSharedPtr<SToolTip> Tooltip;
	Tooltip = IDocumentation::Get()->CreateToolTip(NSLOCTEXT("CompareNode", "CompareNodeRemoveCaseButton_Tooltip", "Remove last case pins"), NULL, GraphNode->GetDocumentationLink(), FString());

	TSharedRef<SButton> RemovePinButton = SNew(SButton)
		.ContentPadding(0.0f)
		.ButtonStyle(FAppStyle::Get(), "NoBorder")
		.OnClicked(this, &SGraphNode_MultiCompareGameplayTag::OnRemovePin)
		.ToolTipText(NSLOCTEXT("CompareNode", "CompareNodeRemovePinButton_Tooltip", "Remove last pin"))
		.ToolTip(Tooltip)
		.Visibility(this, &SGraphNode_MultiCompareGameplayTag::IsRemovePinButtonVisible)
		[
			ButtonContent.ToSharedRef()
		];

	RemovePinButton->SetCursor(EMouseCursor::Hand);

	FMargin AddPinPadding = Settings->GetOutputPinPadding();
	AddPinPadding.Top += 6.0f;

	OutputBox->AddSlot()
		.AutoHeight()
		.VAlign(VAlign_Center)
		.Padding(AddPinPadding)
		[
			RemovePinButton
		];
}

void SGraphNode_MultiCompareGameplayTag::CreateOutputSideAddButton(TSharedPtr<SVerticalBox> OutputBox)
{
	TSharedRef<SWidget> AddPinButton = AddPinButtonContent(
		NSLOCTEXT("CompareNode", "CompareNodeAddPinButton", "Add Case"),
		NSLOCTEXT("CompareNode", "CompareNodeAddPinButton_Tooltip", "Add new case pins"));

	FMargin AddPinPadding = Settings->GetOutputPinPadding();
	AddPinPadding.Top += 6.0f;

	OutputBox->AddSlot()
		.AutoHeight()
		.VAlign(VAlign_Center)
		.Padding(AddPinPadding)
		[
			AddPinButton
		];
}

EVisibility SGraphNode_MultiCompareGameplayTag::IsRemovePinButtonVisible() const
{
	UGameplayTagsK2Node_MultiCompareBase* CompareNode = CastChecked<UGameplayTagsK2Node_MultiCompareBase>(GraphNode);
	if (CompareNode && CompareNode->NumberOfPins > 1)
	{		
		return EVisibility::Visible;
	}
	return EVisibility::Collapsed;
}

EVisibility SGraphNode_MultiCompareGameplayTag::IsAddPinButtonVisible() const
{
	if (GraphNode->IsA<UGameplayTagsK2Node_MultiCompareBase>())
	{
		return SGraphNode::IsAddPinButtonVisible();
	}
	return EVisibility::Collapsed;
}

FReply SGraphNode_MultiCompareGameplayTag::OnRemovePin()
{
	UGameplayTagsK2Node_MultiCompareBase* CompareNode = CastChecked<UGameplayTagsK2Node_MultiCompareBase>(GraphNode);

	const FScopedTransaction Transaction(NSLOCTEXT("CompareNode", "AddExecutionPin", "Add Execution Pin"));
	CompareNode->Modify();

	CompareNode->RemovePin();
	CompareNode->ReconstructNode();
	FBlueprintEditorUtils::MarkBlueprintAsModified(CompareNode->GetBlueprint());

	UpdateGraphNode();
	GraphNode->GetGraph()->NotifyNodeChanged(GraphNode);

	return FReply::Handled();
}

FReply SGraphNode_MultiCompareGameplayTag::OnAddPin()
{
	UGameplayTagsK2Node_MultiCompareBase* CompareNode = CastChecked<UGameplayTagsK2Node_MultiCompareBase>(GraphNode);

	const FScopedTransaction Transaction(NSLOCTEXT("CompareNode", "AddExecutionPin", "Add Execution Pin"));
	CompareNode->Modify();

	CompareNode->AddPin();
	CompareNode->ReconstructNode();
	FBlueprintEditorUtils::MarkBlueprintAsModified(CompareNode->GetBlueprint());

	UpdateGraphNode();
	GraphNode->GetGraph()->NotifyNodeChanged(GraphNode);

	return FReply::Handled();
}
