// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphNode.h"

class SVerticalBox;

class SGraphNode_MultiCompareGameplayTag : public SGraphNode
{
public:
	SLATE_BEGIN_ARGS(SGraphNode_MultiCompareGameplayTag){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, class UGameplayTagsK2Node_MultiCompareBase* InNode);

protected:

	// SGraphNode interface
	virtual void CreateOutputSideAddButton(TSharedPtr<SVerticalBox> OutputBox) override;
	virtual void CreateOutputSideRemoveButton(TSharedPtr<SVerticalBox> OutputBox);
	virtual EVisibility IsAddPinButtonVisible() const override;
	virtual FReply OnAddPin() override;
	// End of SGraphNode interface

	EVisibility IsRemovePinButtonVisible() const;
	FReply OnRemovePin();
};
