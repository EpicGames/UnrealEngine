// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRenameGameplayTagDialog.h"
#include "GameplayTagsEditorModule.h"
#include "GameplayTagsManager.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/SWindow.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"

#define LOCTEXT_NAMESPACE "RenameGameplayTag"

void SRenameGameplayTagDialog::Construct(const FArguments& InArgs)
{
	check(InArgs._GameplayTagNode.IsValid())

	GameplayTagNode = InArgs._GameplayTagNode;
	OnGameplayTagRenamed = InArgs._OnGameplayTagRenamed;

	ChildSlot
	[
		SNew(SBox)
		.Padding(InArgs._Padding)
		[
			SNew(SGridPanel)
			.FillColumn(1, 1.0)

			// Current name display
			+ SGridPanel::Slot(0, 0)
			.Padding(2)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			[
				SNew(STextBlock)
				.Font(FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont")))
				.Text(LOCTEXT("CurrentTag", "Current Tag:"))
			]

			+ SGridPanel::Slot(1, 0)
			.Padding(2)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			[
				SNew(STextBlock)
				.Font(FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont")))
				.Text(FText::FromName(GameplayTagNode->GetCompleteTag().GetTagName()))
			]
			
			// New name controls
			+ SGridPanel::Slot(0, 1)
			.Padding(2)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			[
				SNew(STextBlock )
				.Font(FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont")))
				.Text(LOCTEXT("NewTag", "New Tag:"))
			]

			+ SGridPanel::Slot(1, 1)
			.Padding(2)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			[
				SAssignNew(NewTagNameTextBox, SEditableTextBox)
				.Font(FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont")))
				.Text(FText::FromName(GameplayTagNode->GetSimpleTagName()))
				.OnTextCommitted(this, &SRenameGameplayTagDialog::OnRenameTextCommitted)
			]

			// Dialog controls
			+ SGridPanel::Slot(0, 2)
			.ColumnSpan(2)
			.HAlign(HAlign_Right)
			.Padding(FMargin(0, 16))
			[
				SNew(SHorizontalBox)

				// Rename
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0, 0, 8.0f, 0)
				[
					SNew(SButton)
					.IsFocusable(false)
					.IsEnabled(this, &SRenameGameplayTagDialog::IsRenameEnabled)
					.OnClicked(this, &SRenameGameplayTagDialog::OnRenameClicked)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("RenameTagButtonText", "Rename"))
					]
				]

				// Cancel
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.IsFocusable(false)
					.OnClicked(this, & SRenameGameplayTagDialog::OnCancelClicked)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("CancelRenameButtonText", "Cancel"))
					]
				]
			]
		]
	];
}

bool SRenameGameplayTagDialog::IsRenameEnabled() const
{
	FString CurrentTagText;

	if (NewTagNameTextBox.IsValid())
	{
		CurrentTagText = NewTagNameTextBox->GetText().ToString();
	}

	return !CurrentTagText.IsEmpty() && CurrentTagText != GameplayTagNode->GetCompleteTag().GetTagName().ToString();
}

void SRenameGameplayTagDialog::RenameAndClose()
{
	IGameplayTagsEditorModule& Module = IGameplayTagsEditorModule::Get();

	const FString TagToRename = GameplayTagNode->GetCompleteTag().GetTagName().ToString();
	const FString NewTagName = GameplayTagNode->GetParentTagNode().IsValid() ? GameplayTagNode->GetParentTagNode()->GetCompleteTag().GetTagName().ToString() + '.' + NewTagNameTextBox->GetText().ToString() : NewTagNameTextBox->GetText().ToString();

	if (Module.RenameTagInINI(TagToRename, NewTagName, true))
	{
		OnGameplayTagRenamed.ExecuteIfBound(TagToRename, NewTagName);
	}

	CloseContainingWindow();
}

void SRenameGameplayTagDialog::OnRenameTextCommitted(const FText& InText, ETextCommit::Type InCommitType)
{
	if (InCommitType == ETextCommit::OnEnter && IsRenameEnabled())
	{
		RenameAndClose();
	}
}

FReply SRenameGameplayTagDialog::OnRenameClicked()
{
	RenameAndClose();

	return FReply::Handled();
}

FReply SRenameGameplayTagDialog::OnCancelClicked()
{
	CloseContainingWindow();

	return FReply::Handled();
}

void SRenameGameplayTagDialog::CloseContainingWindow()
{
	TSharedPtr<SWindow> CurrentWindow = FSlateApplication::Get().FindWidgetWindow( AsShared() );

	if (CurrentWindow.IsValid())
	{
		CurrentWindow->RequestDestroyWindow();
	}
}

#undef LOCTEXT_NAMESPACE
