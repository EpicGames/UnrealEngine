// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class SEditableTextBox;
namespace ETextCommit { enum Type : int; }
struct FGameplayTagNode;

class SRenameGameplayTagDialog : public SCompoundWidget
{
public:

	DECLARE_DELEGATE_TwoParams( FOnGameplayTagRenamed, FString/* OldName */, FString/* NewName */);

	SLATE_BEGIN_ARGS( SRenameGameplayTagDialog )
		: _GameplayTagNode()
		, _Padding(FMargin(15))
		{}
		SLATE_ARGUMENT( TSharedPtr<FGameplayTagNode>, GameplayTagNode )		// The gameplay tag we want to rename
		SLATE_EVENT( FOnGameplayTagRenamed, OnGameplayTagRenamed )	// Called when the tag is renamed
		SLATE_ARGUMENT(FMargin, Padding)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:

	/** Checks if we're in a valid state to rename the tag */
	bool IsRenameEnabled() const;

	/** Renames the tag based on dialog parameters */
	FReply OnRenameClicked();

	/** Callback for when Cancel is clicked */
	FReply OnCancelClicked();

	/** Renames the tag and attempts to close the active window */
	void RenameAndClose();

	/** Attempts to rename the tag if enter is pressed while editing the tag name */
	void OnRenameTextCommitted(const FText& InText, ETextCommit::Type InCommitType);

	/** Closes the window that contains this widget */
	void CloseContainingWindow();

private:

	TSharedPtr<FGameplayTagNode> GameplayTagNode;

	TSharedPtr<SEditableTextBox> NewTagNameTextBox;

	FOnGameplayTagRenamed OnGameplayTagRenamed;
};
