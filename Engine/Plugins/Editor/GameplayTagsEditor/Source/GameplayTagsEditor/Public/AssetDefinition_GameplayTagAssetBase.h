// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"

#include "AssetDefinition_GameplayTagAssetBase.generated.h"

#define UE_API GAMEPLAYTAGSEDITOR_API

struct FToolMenuSection;
struct FGameplayTagContainer;

/** Base asset type actions for any classes with gameplay tagging */
UCLASS(MinimalAPI, Abstract)
class UAssetDefinition_GameplayTagAssetBase : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	UAssetDefinition_GameplayTagAssetBase() {};

	/** Traditionally these are implemented in a MenuExtension namespace. However, UAssetDefinition_GameplayTagAssetBase is an abstract class,
	* and the derived classes need to invoke this in their static MenuExtension functions**/
	static UE_API void AddGameplayTagsEditMenuExtension(FToolMenuSection& InSection, TArray<UObject*> InObjects, const FName& OwnedGameplayTagPropertyName);

	// UAssetDefinition Begin
	/** Overridden to specify misc category */
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Misc };
		return Categories;
	}
	// UAssetDefinition End

private:
	/**
	 * Open the gameplay tag editor
	 *
	 * @param TagAssets	Assets to open the editor with
	 */
	static UE_API void OpenGameplayTagEditor(TArray<UObject*> Objects, TArray<FGameplayTagContainer> Containers, const FName& OwnedGameplayTagPropertyName);
};

#undef UE_API
