// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

#define UE_API GAMEPLAYTAGSEDITOR_API

class IPropertyTypeCustomization;
struct FGameplayTag;
struct FGameplayTagContainer;

DECLARE_DELEGATE_OneParam(FOnSetGameplayTag, const FGameplayTag&);
DECLARE_DELEGATE_OneParam(FOnSetGameplayTagContainer, const FGameplayTagContainer&);

class SWidget;

/**
 * The public interface to this module
 */
class IGameplayTagsEditorModule : public IModuleInterface
{

public:

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IGameplayTagsEditorModule& Get()
	{
		return FModuleManager::LoadModuleChecked< IGameplayTagsEditorModule >( "GameplayTagsEditor" );
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded( "GameplayTagsEditor" );
	}

	/** Tries to add a new gameplay tag to the ini lists */
	virtual bool AddNewGameplayTagToINI(const FString& NewTag, const FString& Comment = TEXT(""), FName TagSourceName = NAME_None, bool bIsRestrictedTag = false, bool bAllowNonRestrictedChildren = true) = 0;

	/** Tries to delete a tag from the library. This will pop up special UI or error messages as needed. It will also delete redirectors if that is specified. */
	virtual bool DeleteTagFromINI(TSharedPtr<struct FGameplayTagNode> TagNodeToDelete) = 0;
	virtual void DeleteTagsFromINI(const TArray<TSharedPtr<struct FGameplayTagNode>>& TagNodesToDelete) = 0;

	/** Tries to rename a tag, leaving a rediretor in the ini, and adding the new tag if it does not exist yet */
	virtual bool RenameTagInINI(const FString& TagToRename, const FString& TagToRenameTo, bool bRenameChildren = true) = 0;

	/** Updates info about a tag */
	virtual bool UpdateTagInINI(const FString& TagToUpdate, const FString& Comment, bool bIsRestrictedTag, bool bAllowNonRestrictedChildren) = 0;

	/** Tries to move existing tags from each source ini lists to the target source ini list. */
	virtual bool MoveTagsBetweenINI(const TArray<FString>& TagsToMove, const FName& TargetTagSource, TArray<FString>& OutTagsMoved, TArray<FString>& OutFailedToMoveTags) = 0;

	/** Adds a transient gameplay tag (only valid for the current editor session) */
	virtual bool AddTransientEditorGameplayTag(const FString& NewTransientTag) = 0;

	/** Adds a new tag source, well use project config directory if not specified. This will not save anything until a tag is added */
	virtual bool AddNewGameplayTagSource(const FString& NewTagSource, const FString& RootDirToUse = FString()) = 0;

	/**
	 * Creates a simple version of a tag container widget that has a default value and will call a custom callback
	 * @param OnSetTag			Delegate called when container is changed
	 * @param GameplayTagValue	Shared ptr to tag container value that will be used as default and modified
	 * @param FilterString		Optional filter string, same format as Categories metadata on tag properties
	 */
	virtual TSharedRef<SWidget> MakeGameplayTagContainerWidget(FOnSetGameplayTagContainer OnSetTag, TSharedPtr<FGameplayTagContainer> GameplayTagContainer, const FString& FilterString = FString()) = 0;

	/**
	 * Creates a simple version of a gameplay tag widget that has a default value and will call a custom callback
	 * @param OnSetTag			Delegate called when tag is changed
	 * @param GameplayTagValue	Shared ptr to tag value that will be used as default and modified
	 * @param FilterString		Optional filter string, same format as Categories metadata on tag properties
	 */
	virtual TSharedRef<SWidget> MakeGameplayTagWidget(FOnSetGameplayTag OnSetTag, TSharedPtr<FGameplayTag> GameplayTag, const FString& FilterString = FString()) = 0;

	/** Returns the list of gameplay tags that are not used by content */
	virtual void GetUnusedGameplayTags(TArray<TSharedPtr<struct FGameplayTagNode>>& OutUnusedTags) = 0;
};

/** This is public so that child structs of FGameplayTag can use the details customization */
struct FGameplayTagCustomizationPublic
{
	static UE_API TSharedRef<IPropertyTypeCustomization> MakeInstance();
};

/** This is public so that child structs of FGameplayTagContainer can use the details customization */
struct FGameplayTagContainerCustomizationPublic
{
	static UE_API TSharedRef<IPropertyTypeCustomization> MakeInstance();
};

struct FRestrictedGameplayTagCustomizationPublic
{
	static UE_API TSharedRef<IPropertyTypeCustomization> MakeInstance();
};

#undef UE_API
