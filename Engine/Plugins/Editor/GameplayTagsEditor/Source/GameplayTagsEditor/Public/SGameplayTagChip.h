// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "GameplayTagContainer.h"

class SButton;
class STextBlock;

/**
 * Widget for displaying a single gameplay tag with optional clear button.
 */
class SGameplayTagChip : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET(SGameplayTagChip, SCompoundWidget)

public:

	DECLARE_DELEGATE_RetVal(FReply, FOnClearPressed);
	DECLARE_DELEGATE_RetVal(FReply, FOnEditPressed);
	DECLARE_DELEGATE_RetVal(FReply, FOnNavigate);
	DECLARE_DELEGATE_RetVal_OneParam(FReply, FOnMenu, const FPointerEvent& /*MouseEvent*/);

	SLATE_BEGIN_ARGS(SGameplayTagChip)
		: _EnableNavigation(false)
		, _ReadOnly(false)
		, _IsSelected(true)
		, _ShowClearButton(true)
	{}
		// Callback for when button body is pressed
		SLATE_EVENT(FOnEditPressed, OnEditPressed)

		// Callback for when clear tag button is pressed
		SLATE_EVENT(FOnClearPressed, OnClearPressed)

		// Callback for when button body is pressed with LMB+Ctrl
		SLATE_EVENT(FOnNavigate, OnNavigate)

		// Callback for when button body is pressed with RMB
		SLATE_EVENT(FOnMenu, OnMenu)

		// If true, allow button navigation behavior
		SLATE_ARGUMENT(bool, EnableNavigation)

		// Flag to set if the list is read only
		SLATE_ARGUMENT( bool, ReadOnly )

		// Is true, the chip is displayed as selected.
		SLATE_ATTRIBUTE(bool, IsSelected)

		// If true, shows clear button inside the chip
		SLATE_ATTRIBUTE(bool, ShowClearButton)

		// Tooltip to display
		SLATE_ATTRIBUTE(FText, ToolTipText)

		// Text to display
		SLATE_ATTRIBUTE(FText, Text)

		// Text to display color
		SLATE_ARGUMENT(TOptional<FSlateColor>, TextColor)
	SLATE_END_ARGS();

	GAMEPLAYTAGSEDITOR_API SGameplayTagChip();

	GAMEPLAYTAGSEDITOR_API void Construct(const FArguments& InArgs);
	
private: 

	void UpdatePillStyle();

	TSlateAttribute<bool> IsSelectedAttribute;
	TSlateAttribute<bool> ShowClearButtonAttribute;
	TSlateAttribute<FText> ToolTipTextAttribute;
	TSlateAttribute<FText> TextAttribute;
	TOptional<FSlateColor> TextColor;
	TSharedPtr<SButton> ChipButton;
	TSharedPtr<SButton> ClearButton;
	TSharedPtr<STextBlock> TagText;
	FOnClearPressed OnClearPressed;
	FOnEditPressed OnEditPressed;
	FOnNavigate OnNavigate;
	FOnMenu OnMenu;
	bool bEnableNavigation = false;
	bool bReadOnly = false;
	bool bLastHasIsSelected = false;
};
