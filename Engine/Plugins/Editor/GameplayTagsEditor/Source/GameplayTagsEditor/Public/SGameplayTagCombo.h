// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "GameplayTagContainer.h"
#include "SGameplayTagChip.h"

class IPropertyHandle;
class SMenuAnchor;
class SButton;
class SComboButton;
class SGameplayTagPicker;

/**
 * Widget for editing a Gameplay Tag.
 */
class SGameplayTagCombo : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET_API(SGameplayTagCombo, SCompoundWidget, GAMEPLAYTAGSEDITOR_API)

public:

	DECLARE_DELEGATE_OneParam(FOnTagChanged, const FGameplayTag /*Tag*/)

	SLATE_BEGIN_ARGS(SGameplayTagCombo)
		: _Filter()
		, _ReadOnly(false)
		, _DisplayShortTagNames(false)
		, _EnableNavigation(false)
		, _PropertyHandle(nullptr)
	{}
		// Comma delimited string of tag root names to filter by
		SLATE_ARGUMENT(FString, Filter)

		// The name that will be used for the tag picker settings file
		SLATE_ARGUMENT(FString, SettingsName)

		// Flag to set if the list is read only
		SLATE_ARGUMENT(bool, ReadOnly)

		// Hide Filter from chip display
		SLATE_ARGUMENT(bool, DisplayShortTagNames)

		// If true, allow button navigation behavior
		SLATE_ARGUMENT(bool, EnableNavigation)

		// Tags to edit
		SLATE_ATTRIBUTE(FGameplayTag, Tag)

		// If set, the tag is read from the property, and the property is changed when tag is edited. 
		SLATE_ARGUMENT(TSharedPtr<IPropertyHandle>, PropertyHandle)

		// Callback for when button body is pressed with LMB+Ctrl
		SLATE_EVENT(SGameplayTagChip::FOnNavigate, OnNavigate)

		// Callback for when button body is pressed with RMB
		SLATE_EVENT(SGameplayTagChip::FOnMenu, OnMenu)

		// Called when a tag status changes
		SLATE_EVENT(FOnTagChanged, OnTagChanged)
	SLATE_END_ARGS();

	GAMEPLAYTAGSEDITOR_API SGameplayTagCombo();

	GAMEPLAYTAGSEDITOR_API void Construct(const FArguments& InArgs);

	GAMEPLAYTAGSEDITOR_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:

	bool ShowClearButton() const;
	FText GetText() const;
	bool IsValueEnabled() const;
	FText GetToolTipText() const;
	bool IsSelected() const;
	FReply OnClearPressed();
	void OnMenuOpenChanged(const bool bOpen) const;
	TSharedRef<SWidget> OnGetMenuContent();
	FGameplayTag GetCommonTag() const;
	FReply OnTagMenu(const FPointerEvent& MouseEvent);
	FReply OnEditTag() const;
	void RefreshTagsFromProperty();
	void OnTagSelected(const TArray<FGameplayTagContainer>& TagContainers);
	void OnClearTag();
	void OnCopyTag(const FGameplayTag TagToCopy) const;
	void OnPasteTag();
	bool CanPaste() const;

	TSlateAttribute<FGameplayTag> TagAttribute;
	TArray<FGameplayTag> TagsFromProperty;
	bool bHasMultipleValues = false;
	bool bIsReadOnly = false;
	bool bDisplayShortTagNames = false;
	FString Filter;
	FString SettingsName;
	FOnTagChanged OnTagChanged;
	TSharedPtr<IPropertyHandle> PropertyHandle;
	TSharedPtr<SMenuAnchor> MenuAnchor;
	TSharedPtr<SComboButton> ComboButton;
	TSharedPtr<SGameplayTagPicker> TagPicker;
};
