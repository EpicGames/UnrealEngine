// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "GameplayTagContainer.h"
#include "SGameplayTagChip.h"

class IPropertyHandle;
class SMenuAnchor;
class ITableRow;
class STableViewBase;
class SGameplayTagPicker;
class SComboButton;

/**
 * Widget for editing a Gameplay Tag Container.
 */
class SGameplayTagContainerCombo : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET_API(SGameplayTagContainerCombo, SCompoundWidget, GAMEPLAYTAGSEDITOR_API)
	
public:

	DECLARE_DELEGATE(FOnTagCleared)
	DECLARE_DELEGATE_OneParam(FOnTagContainerComboClosed, const FGameplayTagContainer& /*TagContainer*/)
	DECLARE_DELEGATE_OneParam(FOnTagContainerChanged, const FGameplayTagContainer& /*TagContainer*/)
	// If bound returns optional color to use for this tag text. 
    DECLARE_DELEGATE_RetVal_OneParam(TOptional<FSlateColor>, FOnGetTagTextColor, const FGameplayTag& /*Tag*/);

	SLATE_BEGIN_ARGS(SGameplayTagContainerCombo)
		: _Filter()
		, _ReadOnly(false)
		, _DisplayShortTagNames(false)
		, _EnableNavigation(false)
		, _PropertyHandle(nullptr)
	{}
		// Comma delimited string of tag root names to filter by
		SLATE_ARGUMENT(FString, Filter)

		// The name that will be used for the picker settings file
		SLATE_ARGUMENT(FString, SettingsName)

		// Flag to set if the list is read only
		SLATE_ARGUMENT(bool, ReadOnly)

		// Hide Filter from chip display
		SLATE_ARGUMENT(bool, DisplayShortTagNames)

		// If true, allow button navigation behavior
		SLATE_ARGUMENT(bool, EnableNavigation)

		// Tag container to edit
		SLATE_ATTRIBUTE(FGameplayTagContainer, TagContainer)

		// If set, the tag container is read from the property, and the property is changed when tag container is edited. 
		SLATE_ARGUMENT(TSharedPtr<IPropertyHandle>, PropertyHandle)

		// Callback for when button body is pressed with LMB+Ctrl
		SLATE_EVENT(SGameplayTagChip::FOnNavigate, OnNavigate)

		// Callback for when button body is pressed with RMB
		SLATE_EVENT(SGameplayTagChip::FOnMenu, OnMenu)

		// Called when the tag container combo selection is closed
		SLATE_EVENT(FOnTagCleared, OnTagCleared)

		// Called when the tag container combo selection is closed
		SLATE_EVENT(FOnTagContainerComboClosed, OnTagContainerComboClosed)

		// Called when a tag container changes
		SLATE_EVENT(FOnTagContainerChanged, OnTagContainerChanged)

		// Called to request option tag text color
		SLATE_EVENT(FOnGetTagTextColor, OnGetTagTextColor)
	SLATE_END_ARGS();

	GAMEPLAYTAGSEDITOR_API SGameplayTagContainerCombo();

	GAMEPLAYTAGSEDITOR_API void Construct(const FArguments& InArgs);

	GAMEPLAYTAGSEDITOR_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/** Returns true if the expanded edit UI is open */
	GAMEPLAYTAGSEDITOR_API bool IsOpen() const;

	/** Opens the expanded edit UI, optionally focusing on a specific tag */
	GAMEPLAYTAGSEDITOR_API void Open(FGameplayTag TagToFocus = { });
	
	GAMEPLAYTAGSEDITOR_API void RefreshTagContainers();

private:

	struct FEditableItem
	{
		FEditableItem() = default;
		FEditableItem(const FGameplayTag InTag, const int InCount = 1)
			: Tag(InTag)
			, Count(InCount)
		{
		}
		
		FGameplayTag Tag;
		int32 Count = 0;
		bool bMultipleValues = false;
	};

	bool IsValueEnabled() const;
	TSharedRef<ITableRow> MakeTagListViewRow(TSharedPtr<FEditableItem> Item, const TSharedRef<STableViewBase>& OwnerTable);
	void OnMenuOpenChanged(const bool bOpen);
	TSharedRef<SWidget> OnGetMenuContent();
	FText GetTagToolTipText(const FGameplayTag TagToDisplay) const;

	FReply OnTagMenu(const FPointerEvent& MouseEvent, const FGameplayTag GameplayTag);
	FReply OnEmptyMenu(const FPointerEvent& MouseEvent);
	FReply OnEditClicked(const FGameplayTag TagToHilight);
	FReply OnClearAllClicked();
	FReply OnClearTagClicked(const FGameplayTag TagToClear);
	void OnTagChanged(const TArray<FGameplayTagContainer>& TagContainers);
	void OnClearAll();
	void OnCopyTag(const FGameplayTag TagToCopy) const;
	void OnPasteTag();
	bool CanPaste() const;
	void OnSearchForAnyReferences() const;
	
	TSlateAttribute<FGameplayTagContainer> TagContainerAttribute;
	TArray<FGameplayTagContainer> CachedTagContainers;
	TArray<TSharedPtr<FEditableItem>> TagsToEdit;
	TSharedPtr<SListView<TSharedPtr<FEditableItem>>> TagListView;
	
	FString Filter;
	FString SettingsName;
	bool bIsReadOnly = false;
	bool bDisplayShortTagNames = false;
	FOnTagCleared OnTagCleared;
	FOnTagContainerComboClosed OnTagContainerComboClosed;
	FOnTagContainerChanged OnTagContainerChanged;
	FOnGetTagTextColor OnGetTagTextColor;
	TSharedPtr<IPropertyHandle> PropertyHandle;
	TSharedPtr<SMenuAnchor> MenuAnchor;
	TSharedPtr<SComboButton> ComboButton;
	TSharedPtr<SGameplayTagPicker> TagPicker;
	FGameplayTag TagToHilight;
};