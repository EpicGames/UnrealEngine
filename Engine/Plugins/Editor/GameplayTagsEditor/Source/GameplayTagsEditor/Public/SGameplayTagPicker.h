// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SlateFwd.h"
#include "UObject/Object.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"
#include "GameplayTagsManager.h"

#define UE_API GAMEPLAYTAGSEDITOR_API

class IPropertyHandle;
class SComboButton;
class SAddNewGameplayTagWidget;

/** Determines the behavior of the gameplay tag UI depending on where it's used */
enum class EGameplayTagPickerMode : uint8
{
	SelectionMode,		// Allows selecting tags for gameplay tag containers.
	ManagementMode,		// Allows renaming/deleting tags, does not require gameplay tag containers.
	HybridMode			// Allows doing operations from both Selection and Management mode.
};

/** Widget allowing user to tag assets with gameplay tags */
class SGameplayTagPicker : public SCompoundWidget
{
public:

	/** Called when a tag status is changed */
	DECLARE_DELEGATE_OneParam(FOnTagChanged, const TArray<FGameplayTagContainer>& /*TagContainers*/)

	/** Called on when tags might need refreshing (e.g. after undo/redo or when tags change). Use SetTagContainers() to set the new tags. */
	DECLARE_DELEGATE_OneParam(FOnRefreshTagContainers, SGameplayTagPicker& /*TagPicker*/)
	
	enum class ETagFilterResult
	{
		IncludeTag,
		ExcludeTag
	};

	DECLARE_DELEGATE_RetVal_OneParam(ETagFilterResult, FOnFilterTag, const TSharedPtr<FGameplayTagNode>&)

	SLATE_BEGIN_ARGS(SGameplayTagPicker)
		: _Filter()
		, _SettingsName(TEXT(""))
		, _ReadOnly(false)
		, _MultiSelect(true)
		, _RestrictedTags(false)
		, _ShowMenuItems(false)
		, _PropertyHandle(nullptr)
		, _GameplayTagPickerMode(EGameplayTagPickerMode::SelectionMode)
		, _MaxHeight(260.0f)
		, _Padding(FMargin(2.0f))
	{}
		// Comma delimited string of tag root names to filter by
		SLATE_ARGUMENT(FString, Filter)

		// Optional filter function called when generating the tag list
		SLATE_EVENT(FOnFilterTag, OnFilterTag)

		// The name that will be used for the settings file
		SLATE_ARGUMENT(FString, SettingsName)

		// Flag to set if the list is read only
		SLATE_ARGUMENT(bool, ReadOnly)

		// If we can select multiple entries
		SLATE_ARGUMENT(bool, MultiSelect)

		// If we are dealing with restricted tags or regular gameplay tags
		SLATE_ARGUMENT(bool, RestrictedTags)

		// If set, wraps the picker in a menu builder and adds common menu commands.
		SLATE_ARGUMENT(bool, ShowMenuItems)  

		// Property handle to FGameplayTag or FGameplayTagContainer, used to get and modify the edited tags.
		SLATE_ARGUMENT(TSharedPtr<IPropertyHandle>, PropertyHandle)

		// Tags or tag containers to modify. If MultiSelect is false, the container will container single tags.
		// If PropertyHandle is set, the tag containers will be ignored.
		SLATE_ARGUMENT(TArray<FGameplayTagContainer>, TagContainers)

		// Called when a tag status changes
		SLATE_EVENT(FOnTagChanged, OnTagChanged)

		// Determines behavior of the menu based on where it's used
		SLATE_ARGUMENT(EGameplayTagPickerMode, GameplayTagPickerMode)

		// Caps the height of the gameplay tag tree, 0.0 means uncapped.
		SLATE_ARGUMENT(float, MaxHeight)

		// Padding inside the picker widget
		SLATE_ARGUMENT(FMargin, Padding)

		// Called on when tags might need refreshing (e.g. after undo/redo or when tags change).
		SLATE_EVENT(FOnRefreshTagContainers, OnRefreshTagContainers)
	SLATE_END_ARGS()

	/**
	 * Given a property handle, try and enumerate the tag containers from within it (when dealing with a struct property of type FGameplayTag or FGameplayTagContainer).
	 * @return True if it was possible to enumerate containers (even if no containers were enumerated), or false otherwise.
	 */
	static UE_API bool EnumerateEditableTagContainersFromPropertyHandle(const TSharedRef<IPropertyHandle>& PropHandle, TFunctionRef<bool(const FGameplayTagContainer&)> Callback);

	/**
	 * Given a property handle, try and extract the editable tag containers from within it (when dealing with a struct property of type FGameplayTagContainer).
	 * @return True if it was possible to extract containers (even if no containers were extracted), or false otherwise.
	 */
	static UE_API bool GetEditableTagContainersFromPropertyHandle(const TSharedRef<IPropertyHandle>& PropHandle, TArray<FGameplayTagContainer>& OutEditableContainers);

	UE_API virtual ~SGameplayTagPicker() override;

	/** Construct the actual widget */
	UE_API void Construct(const FArguments& InArgs);
	
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/** Updates the tag list when the filter text changes */
	UE_API void OnFilterTextChanged(const FText& InFilterText);

	/** Returns true if this TagNode has any children that match the current filter */
	UE_API bool FilterChildrenCheck(TSharedPtr<FGameplayTagNode>& InItem) const;

	/** Child recursion helper for FilterChildrenCheck() */
	UE_API bool FilterChildrenCheckRecursive(TSharedPtr<FGameplayTagNode>& InItem) const;

	/** Refreshes the tags that should be displayed by the widget */
	UE_API void RefreshTags();

	/** Forces the widget to refresh its tags on the next tick */
	UE_API void RefreshOnNextTick();

	/** Scrolls the view to specified tag. */
	UE_API void RequestScrollToView(const FGameplayTag Tag);
	
	/** Gets the widget to focus once the menu opens. */
	UE_API TSharedPtr<SWidget> GetWidgetToFocusOnOpen();

	/** Sets tag containers to edit. */
	UE_API void SetTagContainers(TConstArrayView<FGameplayTagContainer> TagContainers);
	
private:

	enum class EGameplayTagAdd : uint8
	{
		Root,
		Child,
		Duplicate,
	};

	UE_API void OnPostUndoRedo();

	/** Verify the tags are all valid and if not prompt the user. */
	UE_API void VerifyAssetTagValidity();

	/* Filters the tree view based on the current filter text. */
	UE_API void FilterTagTree();

	/* string that sets the section of the ini file to use for this class*/ 
	static UE_API const FString SettingsIniSection;

	/* Filename of ini file to store state of the UI */
	static UE_API const FString& GetGameplayTagsEditorStateIni();

	/**
	 * Generate a row widget for the specified item node and table
	 * 
	 * @param InItem		Tag node to generate a row widget for
	 * @param OwnerTable	Table that owns the row
	 * 
	 * @return Generated row widget for the item node
	 */
	UE_API TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FGameplayTagNode> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/**
	 * Get children nodes of the specified node
	 * 
	 * @param InItem		Node to get children of
	 * @param OutChildren	[OUT] Array of children nodes, if any
	 */
	UE_API void OnGetChildren(TSharedPtr<FGameplayTagNode> InItem, TArray< TSharedPtr<FGameplayTagNode> >& OutChildren);

	/**
	 * Called via delegate when the status of a check box in a row changes
	 * 
	 * @param NewCheckState	New check box state
	 * @param NodeChanged	Node that was checked/unchecked
	 */
	UE_API void OnTagCheckStatusChanged(ECheckBoxState NewCheckState, TSharedPtr<FGameplayTagNode> NodeChanged);

	/**
	 * Called via delegate to determine the checkbox state of the specified node
	 * 
	 * @param Node	Node to find the checkbox state of
	 * 
	 * @return Checkbox state of the specified node
	 */
	UE_API ECheckBoxState IsTagChecked(TSharedPtr<FGameplayTagNode> Node) const;

	/**
	 * @return true if the exact Tag provided is included in any of the tag containers the widget is editing.
	 */
	UE_API bool IsExactTagInCollection(TSharedPtr<FGameplayTagNode> Node) const;

	/**
	 * Called via delegate when the status of the allow non-restricted children check box in a row changes
	 *
	 * @param NewCheckState	New check box state
	 * @param NodeChanged	Node that was checked/unchecked
	 */
	UE_API void OnAllowChildrenTagCheckStatusChanged(ECheckBoxState NewCheckState, TSharedPtr<FGameplayTagNode> NodeChanged);

	/**
	 * Called via delegate to determine the non-restricted children checkbox state of the specified node
	 *
	 * @param Node	Node to find the non-restricted children checkbox state of
	 *
	 * @return Non-restricted children heckbox state of the specified node
	 */
	UE_API ECheckBoxState IsAllowChildrenTagChecked(TSharedPtr<FGameplayTagNode> Node) const;

	/** Helper function to determine the visibility of the checkbox for allowing non-restricted children of restricted gameplay tags */
	UE_API EVisibility DetermineAllowChildrenVisible(TSharedPtr<FGameplayTagNode> Node) const;

	/**
	 * Helper function called when the specified node is checked
	 * 
	 * @param NodeChecked	Node that was checked by the user
	 */
	UE_API void OnTagChecked(TSharedPtr<FGameplayTagNode> NodeChecked);

	/**
	 * Helper function called when the specified node is unchecked
	 * 
	 * @param NodeUnchecked	Node that was unchecked by the user
	 */
	UE_API void OnTagUnchecked(TSharedPtr<FGameplayTagNode> NodeUnchecked);

	/**
	 * Recursive function to uncheck all child tags
	 * 
	 * @param NodeUnchecked	Node that was unchecked by the user
	 * @param EditableContainer The container we are removing the tags from
	 */
	UE_API void UncheckChildren(TSharedPtr<FGameplayTagNode> NodeUnchecked, FGameplayTagContainer& EditableContainer);

	/**
	 * Called via delegate to determine the text colour of the specified node
	 *
	 * @param Node	Node to find the colour of
	 *
	 * @return Text colour of the specified node
	 */
	UE_API FSlateColor GetTagTextColour(TSharedPtr<FGameplayTagNode> Node) const;

	/** Called when the user clicks the "Create and Manage Gameplay Tags" button; Opens Gameplay Tag manager window. */
	UE_API void OnManageTagsClicked(TSharedPtr<FGameplayTagNode> Node, TSharedPtr<SComboButton> OwnerCombo);

	/** Called when the user clicks the "Clear All" button; Clears all tags */
	UE_API void OnClearAllClicked(TSharedPtr<SComboButton> OwnerCombo);

	/** Called when the user clicks the "Expand All" button; Expands the entire tag tree */
	UE_API void OnExpandAllClicked(TSharedPtr<SComboButton> OwnerCombo);

	/** Called when the user clicks the "Collapse All" button; Collapses the entire tag tree */
	UE_API void OnCollapseAllClicked(TSharedPtr<SComboButton> OwnerCombo);

	/**
	 * Helper function to set the expansion state of the tree widget
	 * 
	 * @param bExpand If true, expand the entire tree; Otherwise, collapse the entire tree
	 * @param bPersistExpansion If true, persist the expansion state.
	 */
	UE_API void SetTagTreeItemExpansion(bool bExpand, bool bPersistExpansion = false);

	/**
	 * Helper function to set the expansion state of a specific node
	 * 
	 * @param Node		Node to set the expansion state of
	 * @param bExpand	If true, expand the node; Otherwise, collapse the node
	 * @param bPersistExpansion If true, persist the expansion state.
	 */
	UE_API void SetTagNodeItemExpansion(TSharedPtr<FGameplayTagNode> Node, bool bExpand, bool bPersistExpansion = false);

	/** Load settings for the tags*/
	UE_API void LoadSettings();

	/** Migrate settings */
	UE_API void MigrateSettings();

	/** Helper function to determine the visibility of the expandable UI controls */
	UE_API EVisibility DetermineExpandableUIVisibility() const;

	/** Helper function to determine the visibility of the Add New Tag widget */
	UE_API bool CanAddNewTag() const;

	/** Helper function to determine the visibility of the Add New Subtag widget */
	UE_API bool CanAddNewSubTag(TSharedPtr<FGameplayTagNode> Node) const;

	/** Helper function to determine if tag can be modified */
	UE_API bool CanModifyTag(TSharedPtr<FGameplayTagNode> Node) const;

	/** Recursive load function to go through all tags in the tree and set the expansion*/
	UE_API void LoadTagNodeItemExpansion(TSharedPtr<FGameplayTagNode> Node );

	/** Recursive function to go through all tags in the tree and set the expansion to default*/
	UE_API void SetDefaultTagNodeItemExpansion(TSharedPtr<FGameplayTagNode> Node);

	/** Expansion changed callback */
	UE_API void OnExpansionChanged(TSharedPtr<FGameplayTagNode> InItem, bool bIsExpanded);

	/** Callback for when a new tag is added */
	UE_API void OnGameplayTagAdded(const FString& TagName, const FString& TagComment, const FName& TagSource);

	/** Initializes the inline add tag widget and makes in visible. */
	UE_API void ShowInlineAddTagWidget(const EGameplayTagAdd Mode, TSharedPtr<FGameplayTagNode> InTagNode = TSharedPtr<FGameplayTagNode>());

	/** Returns true if the passed in node is the root node, and there is only one node on the root level */
	UE_API bool IsNodeSingleRoot(const TSharedPtr<FGameplayTagNode>& InNode) const;

	UE_API FText GetHighlightText() const;
	
	/** Creates a dropdown menu to provide additional functionality for tags (renaming, deletion, search for references, etc.) */
	UE_API TSharedRef<SWidget> MakeTagActionsMenu(TSharedPtr<FGameplayTagNode> InTagNode, TSharedPtr<SComboButton> OwnerCombo, const bool bInShouldCloseWindowAfterMenuSelection);

	/** Creates settings menu content. */
	UE_API TSharedRef<SWidget> MakeSettingsMenu(TSharedPtr<SComboButton> OwnerCombo);

	UE_API void OnAddSubTag(TSharedPtr<FGameplayTagNode> InTagNode, TSharedPtr<SComboButton> OwnerCombo);

	UE_API void OnDuplicateTag(TSharedPtr<FGameplayTagNode> InTagNode, TSharedPtr<SComboButton> OwnerCombo);
	
	/** Attempts to rename the tag through a dialog box */
	UE_API void OnRenameTag(TSharedPtr<FGameplayTagNode> InTagNode, TSharedPtr<SComboButton> OwnerCombo);

	/** Attempts to delete the specified tag */
	UE_API void OnDeleteTag(TSharedPtr<FGameplayTagNode> InTagNode, TSharedPtr<SComboButton> OwnerCombo);

	/** Attempts to select the exact specified tag*/
	UE_API void OnSelectExactTag(TSharedPtr<FGameplayTagNode> InTagNode, TSharedPtr<SComboButton> OwnerCombo);

	/** Attempts to unselect the specified tag, but not the children */
	UE_API void OnUnselectExactTag(TSharedPtr<FGameplayTagNode> InTagNode, TSharedPtr<SComboButton> OwnerCombo);

	/** Searches for all references for the selected tag */
	UE_API void OnSearchForReferences(TSharedPtr<FGameplayTagNode> InTagNode, TSharedPtr<SComboButton> OwnerCombo);

	/** Searches for all references for the selected tag and its children */
	UE_API void OnSearchForReferencesAsChildren(TSharedPtr<FGameplayTagNode> InTagNode, TSharedPtr<SComboButton> OwnerCombo);

	/** Copies individual tag's name to clipboard. */
	UE_API void OnCopyTagNameToClipboard(TSharedPtr<FGameplayTagNode> InTagNode, TSharedPtr<SComboButton> OwnerCombo);
	
	/** Returns true if the user can select tags from the widget */
	UE_API bool CanSelectTags() const;

	/** Called when tags are selected. */
	UE_API void OnContainersChanged();

	/** Opens a dialog window to rename the selected tag */
	UE_API void OpenRenameGameplayTagDialog(TSharedPtr<FGameplayTagNode> GameplayTagNode) const;

	/** Delegate that is fired when a tag is successfully renamed */
	UE_API void OnGameplayTagRenamed(FString OldTagName, FString NewTagName);

	/** Populate tag items from the gameplay tags manager. */
	UE_API void GetFilteredGameplayRootTags(const FString& InFilterString, TArray<TSharedPtr<FGameplayTagNode>>& OutNodes) const;

	/** Called to create context menu for the tag tree items. */
	UE_API TSharedPtr<SWidget> OnTreeContextMenuOpening();

	/** Called when the tag tree selection changes. */
	UE_API void OnTreeSelectionChanged(TSharedPtr<FGameplayTagNode> SelectedItem, ESelectInfo::Type SelectInfo);

	/** Called to handle key presses in the tag tree. */
	UE_API FReply OnTreeKeyDown(const FGeometry& Geometry, const FKeyEvent& Key);
	
	/* Holds the Name of this TagContainer used for saving out expansion settings */
	FString SettingsName;

	/* Filter string tokens used during search box */
	TArray<FString> FilterStringTokens;

	/** root filter (passed in on creation) */
	FString RootFilterString;

	/** User specified filter function. */
	FOnFilterTag TagFilter; 

	/* Flag to set if the list is read only*/
	bool bReadOnly = false;

	/* Flag to set if we can select multiple items form the list*/
	bool bMultiSelect = true;

	/** If true, refreshes tags on the next frame */
	bool bDelayRefresh = false;

	/** If true, this widget is displaying restricted tags; if false this widget displays regular gameplay tags. */
	bool bRestrictedTags = false;

	/** The maximum height of the gameplay tag tree. If 0, the height is unbound. */
	float MaxHeight = 260.0f;

	/* Array of all tags */
	TArray<TSharedPtr<FGameplayTagNode>> TagItems;

	/* Array of tags filtered to be displayed in the TreeView */
	TArray<TSharedPtr<FGameplayTagNode>> FilteredTagItems;

	/** Container widget holding the tag tree */
	TSharedPtr<SBorder> TagTreeContainerWidget;

	/** Tree widget showing the gameplay tag library */
	TSharedPtr<STreeView<TSharedPtr<FGameplayTagNode>>> TagTreeWidget;

	/** Allows for the user to find a specific gameplay tag in the tree */
	TSharedPtr<SSearchBox> SearchTagBox;

	/** Property to edit */
	TSharedPtr<IPropertyHandle> PropertyHandle;

	/** Containers to modify, ignored if PropertyHandle is set. */
	TArray<FGameplayTagContainer> TagContainers;

	/** Called when the Tag list changes*/
	FOnTagChanged OnTagChanged;

	/** Called on when tags might need refreshing (e.g. after undo/redo or when tags change). */
	FOnRefreshTagContainers OnRefreshTagContainers;

	/** Tag to scroll to, cleared after scroll is requested. */
	FGameplayTag RequestedScrollToTag;
	
	/** Determines behavior of the widget */
	EGameplayTagPickerMode GameplayTagPickerMode = EGameplayTagPickerMode::SelectionMode;

	/** Guard value used to prevent feedback loops in selection handling. */
	bool bInSelectionChanged = false;

	/** Guard value used define if expansion code should persist the expansion operations. */
	bool bPersistExpansionChange = true;

	/** Expanded items cached from settings */
	TSet<TSharedPtr<FGameplayTagNode>> CachedExpandedItems;

	TSharedPtr<SAddNewGameplayTagWidget> AddNewTagWidget;
	bool bNewTagWidgetVisible = false;

	FDelegateHandle PostUndoRedoDelegateHandle;
};


struct FGameplayTagManagerWindowArgs
{
	FText Title;
	FString Filter; // Comma delimited string of tag root names to filter by
	FGameplayTag HighlightedTag; // Tag to highlight when window is opened. 
	bool bRestrictedTags;
};

namespace UE::GameplayTags::Editor
{
	GAMEPLAYTAGSEDITOR_API TWeakPtr<SGameplayTagPicker> OpenGameplayTagManager(const FGameplayTagManagerWindowArgs& Args);
	GAMEPLAYTAGSEDITOR_API TSharedRef<SGameplayTagPicker> Create(const FGameplayTagManagerWindowArgs& Args);
};

#undef UE_API
