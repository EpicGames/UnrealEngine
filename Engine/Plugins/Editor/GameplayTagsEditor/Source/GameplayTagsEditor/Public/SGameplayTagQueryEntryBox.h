// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "GameplayTagContainer.h"

class IPropertyHandle;
class SHorizontalBox;
class SGameplayTagQueryWidget;

/**
 * Widget for editing a Gameplay Tag Query.
 */
class SGameplayTagQueryEntryBox : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET_API(SGameplayTagQueryEntryBox, SCompoundWidget, GAMEPLAYTAGSEDITOR_API)
	
public:

	DECLARE_DELEGATE_OneParam(FOnTagQueryChanged, const FGameplayTagQuery& /*TagQuery*/)

	SLATE_BEGIN_ARGS(SGameplayTagQueryEntryBox)
		: _Filter()
		, _ReadOnly(false)
		, _DescriptionMaxWidth(250.0f)
		, _PropertyHandle(nullptr)
	{}
		/** Comma delimited string of tag root names to filter by */
		SLATE_ARGUMENT(FString, Filter)

		/** Flag to set if the list is read only */
		SLATE_ARGUMENT(bool, ReadOnly)

		/** Maximum with of the query description field. */
		SLATE_ARGUMENT(float, DescriptionMaxWidth)

		/** Tag query to edit */
		SLATE_ATTRIBUTE(FGameplayTagQuery, TagQuery)

		/** If set, the tag query is read from the property, and the property is update when tag query is edited. */ 
		SLATE_ARGUMENT(TSharedPtr<IPropertyHandle>, PropertyHandle)

		SLATE_EVENT(FOnTagQueryChanged, OnTagQueryChanged)
	SLATE_END_ARGS();

	GAMEPLAYTAGSEDITOR_API SGameplayTagQueryEntryBox();

	GAMEPLAYTAGSEDITOR_API void Construct(const FArguments& InArgs);

	GAMEPLAYTAGSEDITOR_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	
private:

	void CacheQueryList();
	struct FGameplayTagQueryWindowArgs BuildWindowArgs();
	FReply OnEditButtonClicked();
	FReply OnClearAllButtonClicked();
	EVisibility GetQueryDescVisibility() const;
	bool HasAnyValidQueries() const;
	bool IsValueEnabled() const;
	FText GetQueryDescText() const;
	FText GetQueryDescTooltip() const;
	void OnQueriesCommitted(const TArray<FGameplayTagQuery>& TagQueries);
	
	TSlateAttribute<FGameplayTagQuery> TagQueryAttribute;

	bool bIsReadOnly = false;
	FString Filter;
	FOnTagQueryChanged OnTagQueryChanged;
	TSharedPtr<IPropertyHandle> PropertyHandle;
	TArray<FGameplayTagQuery> CachedQueries;

	TSharedPtr<SHorizontalBox> WidgetContainer;

	TWeakPtr<SGameplayTagQueryWidget> QueryWidget;
	
	FText QueryDescription;
	FText QueryDescriptionTooltip;
};
