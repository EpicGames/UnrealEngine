// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/AppStyle.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API GAMEPLAYTAGSEDITOR_API

class ITableRow;
class SSearchBox;
class STableViewBase;
enum class ECheckBoxState : uint8;
struct FGameplayTagContainer;
struct FGameplayTagNode;
template <typename ItemType> class STreeView;

class IPropertyHandle;
class SAddNewGameplayTagWidget;
class SAddNewGameplayTagSourceWidget;

/** Determines the behavior of the gameplay tag UI depending on where it's used */
enum class EGameplayTagUIMode : uint8
{
	SelectionMode,    // Allows selecting tags for gameplay tag containers.
	ManagementMode,   // Allows renaming/deleting tags, does not require gameplay tag containers.
	HybridMode        // Allows doing operations from both Selection and Management mode.
};

/** Widget allowing user to tag assets with gameplay tags */
class SGameplayTagWidget : public SCompoundWidget
{
public:

	/** Called when a tag status is changed */
	DECLARE_DELEGATE( FOnTagChanged )
	enum class ETagFilterResult
	{
		IncludeTag,
		ExcludeTag
	};
	
	DECLARE_DELEGATE_RetVal_OneParam( ETagFilterResult, FOnFilterTag, const TSharedPtr<FGameplayTagNode>&)

	SLATE_BEGIN_ARGS( SGameplayTagWidget )
		: _Padding(2.f)
		, _Filter()
		, _NewTagName(TEXT(""))
		, _ReadOnly( false )
		, _TagContainerName( TEXT("") )
		, _MultiSelect( true )
		, _NewTagControlsInitiallyExpanded( false )
		, _PropertyHandle( NULL )
		, _GameplayTagUIMode( EGameplayTagUIMode::SelectionMode )
		, _MaxHeight(260.0f)
		, _RestrictedTags( false )
		, _ForceHideAddNewTag(false)
		, _ForceHideAddNewTagSource(false)
		, _ForceHideTagTreeControls(false)
		, _BackgroundBrush(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		, _TagTreeViewBackgroundBrush(nullptr)
		
	{}
		SLATE_ATTRIBUTE( FMargin, Padding ) // Padding for the containing border.
		SLATE_ARGUMENT( FString, Filter ) // Comma delimited string of tag root names to filter by
		SLATE_EVENT( FOnFilterTag, OnFilterTag ) // Optional filter function called when generating the tag list
		SLATE_ARGUMENT( FString, NewTagName ) // String that will initially populate the New Tag Name field
		SLATE_ARGUMENT( bool, ReadOnly ) // Flag to set if the list is read only
		SLATE_ARGUMENT( FString, TagContainerName ) // The name that will be used for the settings file
		SLATE_ARGUMENT( bool, MultiSelect ) // If we can select multiple entries
		SLATE_ARGUMENT( bool, NewTagControlsInitiallyExpanded ) // If the create new tag controls are initially expanded
		SLATE_ARGUMENT( TSharedPtr<IPropertyHandle>, PropertyHandle )
		SLATE_EVENT( FOnTagChanged, OnTagChanged ) // Called when a tag status changes
		SLATE_ARGUMENT( EGameplayTagUIMode, GameplayTagUIMode )	// Determines behavior of the menu based on where it's used
		SLATE_ARGUMENT( float, MaxHeight )	// caps the height of the gameplay tag tree
		SLATE_ARGUMENT( bool, RestrictedTags ) // if we are dealing with restricted tags or regular gameplay tags
		SLATE_ARGUMENT( bool, ForceHideAddNewTag ) // Allow caller context to manually manipulate visibility of add new tag widget
		SLATE_ARGUMENT( bool, ForceHideAddNewTagSource ) // Allow caller context to manually manipulate visibility of add new tag source widget
		SLATE_ARGUMENT( bool, ForceHideTagTreeControls ) // Allow caller context to manually manipulate visibility of Collapse/Expand buttons and filter widget
		SLATE_ARGUMENT( const FSlateBrush*, BackgroundBrush) // Background brush for the whole widget
		SLATE_ARGUMENT( const FSlateBrush*, TagTreeViewBackgroundBrush) // Background brush for the whole widget
	SLATE_END_ARGS()

	/** Simple struct holding a tag container and its owner for generic re-use of the widget */
	struct FEditableGameplayTagContainerDatum
	{
		/** Constructor */
		FEditableGameplayTagContainerDatum(class UObject* InOwnerObj, struct FGameplayTagContainer* InTagContainer)
			: TagContainerOwner(InOwnerObj)
			, TagContainer(InTagContainer)
		{}

		/** Owning UObject of the container being edited */
		TWeakObjectPtr<class UObject> TagContainerOwner;

		/** Tag container to edit */
		struct FGameplayTagContainer* TagContainer; 
	};

	/**
	 * Given a property handle, try and enumerate the editable tag containers from within it (when dealing with a struct property of type FGameplayTagContainer).
	 * @return True if it was possible to enumerate containers (even if no containers were enumerated), or false otherwise.
	 */
	static UE_API bool EnumerateEditableTagContainersFromPropertyHandle(const TSharedRef<IPropertyHandle>& PropHandle, TFunctionRef<bool(const FEditableGameplayTagContainerDatum&)> Callback);

	/**
	 * Given a property handle, try and extract the editable tag containers from within it (when dealing with a struct property of type FGameplayTagContainer).
	 * @return True if it was possible to extract containers (even if no containers were extracted), or false otherwise.
	 */
	static UE_API bool GetEditableTagContainersFromPropertyHandle(const TSharedRef<IPropertyHandle>& PropHandle, TArray<FEditableGameplayTagContainerDatum>& OutEditableContainers);

	/** Construct the actual widget */
	UE_API void Construct(const FArguments& InArgs, const TArray<FEditableGameplayTagContainerDatum>& EditableTagContainers);

	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/** Ensures that this widget will always account for the MaxHeight if it's specified */
	UE_API virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;

	/** Updates the tag list when the filter text changes */
	UE_API void OnFilterTextChanged( const FText& InFilterText );

	/** Returns true if this TagNode has any children that match the current filter */
	UE_API bool FilterChildrenCheck( TSharedPtr<FGameplayTagNode>  );

	/** Returns true if we're currently adding a new tag to an INI file */
	UE_API bool IsAddingNewTag() const;

	/** Refreshes the tags that should be displayed by the widget */
	UE_API void RefreshTags();

	/** Forces the widget to refresh its tags on the next tick */
	UE_API void RefreshOnNextTick();

	/** Gets the widget to focus once the menu opens. */
	UE_API TSharedPtr<SWidget> GetWidgetToFocusOnOpen();

private:

	/** Verify the tags are all valid and if not prompt the user. */
	UE_API void VerifyAssetTagValidity();

	/* Filters the tree view based on the current filter text. */
	UE_API void FilterTagTree();

	/* string that sets the section of the ini file to use for this class*/ 
	static UE_API const FString SettingsIniSection;

	/* Filename of ini file to store state of the UI */
	static UE_API const FString& GetGameplayTagsEditorStateIni();

	/* Holds the Name of this TagContainer used for saving out expansion settings */
	FString TagContainerName;

	/* Filter string used during search box */
	FString FilterString;

	/** root filter (passed in on creation) */
	FString RootFilterString;

	/** User specified filter function. */
	FOnFilterTag TagFilter; 

	/* Flag to set if the list is read only*/
	bool bReadOnly;

	/* Flag to set if we can select multiple items form the list*/
	bool bMultiSelect;

	/** Tracks if the Add Tag UI is expanded */
	bool bAddTagSectionExpanded;

	/** Tracks if the Add Source UI is expanded */
	bool bAddSourceSectionExpanded;

	/** If true, refreshes tags on the next frame */
	bool bDelayRefresh;

	/** If true, this widget is displaying restricted tags; if false this widget displays regular gameplay tags. */
	bool bRestrictedTags;

	/** If true, this widget is unable to display the 'Add new tag' widget */
	bool bForceHideAddNewTag;

	/** If true, this widget is unable to display the 'Add new tag source' widget */
	bool bForceHideAddNewTagSource;

	/** If true, this widget is unable to display the tag tree controls. */
	bool bForceHideTagTreeControls;

	/** The maximum height of the gameplay tag tree. If 0, the height is unbound. */
	float MaxHeight;

	/* Array of tags to be displayed in the TreeView*/
	TArray< TSharedPtr<FGameplayTagNode> > TagItems;

	/* Array of tags to be displayed in the TreeView*/
	TArray< TSharedPtr<FGameplayTagNode> > FilteredTagItems;

	/** Container widget holding the tag tree */
	TSharedPtr<SWidget> TagTreeContainerWidget;

	/** Tree widget showing the gameplay tag library */
	TSharedPtr< STreeView< TSharedPtr<FGameplayTagNode> > > TagTreeWidget;

	/** The widget that controls how new gameplay tags are added to the config files */
	TSharedPtr<class SAddNewGameplayTagWidget> AddNewTagWidget;

	/** The widget that controls how new gameplay tag source files are added */
	TSharedPtr<class SAddNewGameplayTagSourceWidget> AddNewTagSourceWidget;

	/** Allows for the user to find a specific gameplay tag in the tree */
	TSharedPtr<SSearchBox> SearchTagBox;

	/** Containers to modify */
	TArray<FEditableGameplayTagContainerDatum> TagContainers;

	/** Called when the Tag list changes*/
	FOnTagChanged OnTagChanged;

	/** Determines behavior of the widget */
	EGameplayTagUIMode GameplayTagUIMode;

	TSharedPtr<IPropertyHandle> PropertyHandle;

	/**
	 * Generate a row widget for the specified item node and table
	 * 
	 * @param InItem		Tag node to generate a row widget for
	 * @param OwnerTable	Table that owns the row
	 * 
	 * @return Generated row widget for the item node
	 */
	UE_API TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FGameplayTagNode> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/**
	 * Get children nodes of the specified node
	 * 
	 * @param InItem		Node to get children of
	 * @param OutChildren	[OUT] Array of children nodes, if any
	 */
	UE_API void OnGetChildren(TSharedPtr<FGameplayTagNode> InItem, TArray< TSharedPtr<FGameplayTagNode> >& OutChildren);

	/**
	 * Called via delegate when the status of a check box in a row changes
	 * 
	 * @param NewCheckState	New check box state
	 * @param NodeChanged	Node that was checked/unchecked
	 */
	UE_API void OnTagCheckStatusChanged(ECheckBoxState NewCheckState, TSharedPtr<FGameplayTagNode> NodeChanged);

	/**
	 * Called via delegate to determine the checkbox state of the specified node
	 * 
	 * @param Node	Node to find the checkbox state of
	 * 
	 * @return Checkbox state of the specified node
	 */
	UE_API ECheckBoxState IsTagChecked(TSharedPtr<FGameplayTagNode> Node) const;

	/**
	 * @return true if the exact Tag provided is included in any of the tag containers the widget is editing.
	 */
	UE_API bool IsExactTagInCollection(TSharedPtr<FGameplayTagNode> Node) const;

	/**
	 * Called via delegate when the status of the allow non-restricted children check box in a row changes
	 *
	 * @param NewCheckState	New check box state
	 * @param NodeChanged	Node that was checked/unchecked
	 */
	UE_API void OnAllowChildrenTagCheckStatusChanged(ECheckBoxState NewCheckState, TSharedPtr<FGameplayTagNode> NodeChanged);

	/**
	 * Called via delegate to determine the non-restricted children checkbox state of the specified node
	 *
	 * @param Node	Node to find the non-restricted children checkbox state of
	 *
	 * @return Non-restricted children heckbox state of the specified node
	 */
	UE_API ECheckBoxState IsAllowChildrenTagChecked(TSharedPtr<FGameplayTagNode> Node) const;

	/** Helper function to determine the visibility of the checkbox for allowing non-restricted children of restricted gameplay tags */
	UE_API EVisibility DetermineAllowChildrenVisible(TSharedPtr<FGameplayTagNode> Node) const;

	/**
	 * Helper function called when the specified node is checked
	 * 
	 * @param NodeChecked	Node that was checked by the user
	 */
	UE_API void OnTagChecked(TSharedPtr<FGameplayTagNode> NodeChecked);

	/**
	 * Helper function called when the specified node is unchecked
	 * 
	 * @param NodeUnchecked	Node that was unchecked by the user
	 */
	UE_API void OnTagUnchecked(TSharedPtr<FGameplayTagNode> NodeUnchecked);

	/**
	 * Recursive function to uncheck all child tags
	 * 
	 * @param NodeUnchecked	Node that was unchecked by the user
	 * @param EditableContainer The container we are removing the tags from
	 */
	UE_API void UncheckChildren(TSharedPtr<FGameplayTagNode> NodeUnchecked, FGameplayTagContainer& EditableContainer);

	/**
	 * Called via delegate to determine the text colour of the specified node
	 *
	 * @param Node	Node to find the colour of
	 *
	 * @return Text colour of the specified node
	 */
	UE_API FSlateColor GetTagTextColour(TSharedPtr<FGameplayTagNode> Node) const;

	/** Called when the user clicks the "Clear All" button; Clears all tags */
	UE_API FReply OnClearAllClicked();

	/** Called when the user clicks the "Expand All" button; Expands the entire tag tree */
	UE_API FReply OnExpandAllClicked();

	/** Called when the user clicks the "Collapse All" button; Collapses the entire tag tree */
	UE_API FReply OnCollapseAllClicked();

	/**
	 * Helper function to set the expansion state of the tree widget
	 * 
	 * @param bExpand If true, expand the entire tree; Otherwise, collapse the entire tree
	 */
	UE_API void SetTagTreeItemExpansion(bool bExpand);

	/**
	 * Helper function to set the expansion state of a specific node
	 * 
	 * @param Node		Node to set the expansion state of
	 * @param bExapnd	If true, expand the node; Otherwise, collapse the node
	 */
	UE_API void SetTagNodeItemExpansion(TSharedPtr<FGameplayTagNode> Node, bool bExpand);

	/** Load settings for the tags*/
	UE_API void LoadSettings();

	/** Migrate settings */
	UE_API void MigrateSettings();

	/** Helper function to determine the visibility of the expandable UI controls */
	UE_API EVisibility DetermineExpandableUIVisibility() const;

	/** Helper function to determine the visibility of the add new source expandable UI controls */
	UE_API EVisibility DetermineAddNewSourceExpandableUIVisibility() const;

	/** Helper function to determine the visibility of the Add New Tag widget */
	UE_API EVisibility DetermineAddNewTagWidgetVisibility() const;

	/** Helper function to determine the visibility of the Add New Tag Source widget */
	UE_API EVisibility DetermineAddNewSourceWidgetVisibility() const;

	/** Helper function to determine the visibility of the tag tree controls. */
	UE_API EVisibility DetermineTagTreeControlsVisibility() const;

	/** Helper function to determine the visibility of the Add New Subtag widget */
	UE_API EVisibility DetermineAddNewSubTagWidgetVisibility(TSharedPtr<FGameplayTagNode> Node) const;

	/** Helper function to determine the visibility of the Clear Selection button */
	UE_API EVisibility DetermineClearSelectionVisibility() const;

	/** Recursive load function to go through all tags in the tree and set the expansion*/
	UE_API void LoadTagNodeItemExpansion( TSharedPtr<FGameplayTagNode> Node );

	/** Recursive function to go through all tags in the tree and set the expansion to default*/
	UE_API void SetDefaultTagNodeItemExpansion( TSharedPtr<FGameplayTagNode> Node );

	/** Expansion changed callback */
	UE_API void OnExpansionChanged( TSharedPtr<FGameplayTagNode> InItem, bool bIsExpanded );

	/** Callback for when a new tag is added */
	UE_API void OnGameplayTagAdded(const FString& TagName, const FString& TagComment, const FName& TagSource);

	/** Callback when the user wants to add a subtag to an existing tag */
	UE_API FReply OnAddSubtagClicked(TSharedPtr<FGameplayTagNode> InTagNode);

	/** Creates a dropdown menu to provide additional functionality for tags (renaming, deletion, search for references, etc.) */
	UE_API TSharedRef<SWidget> MakeTagActionsMenu(TSharedPtr<FGameplayTagNode> InTagNode);

	/** Attempts to rename the tag through a dialog box */
	UE_API void OnRenameTag(TSharedPtr<FGameplayTagNode> InTagNode);

	/** Attempts to delete the specified tag */
	UE_API void OnDeleteTag(TSharedPtr<FGameplayTagNode> InTagNode);

	/** Attempts to add the exact specified tag*/
	UE_API void OnAddTag(TSharedPtr<FGameplayTagNode> InTagNode);

	/** Attempts to remove the specified tag, but not the children */
	UE_API void OnRemoveTag(TSharedPtr<FGameplayTagNode> InTagNode);

	/** Searches for all references for the selected tag */
	UE_API void OnSearchForReferences(TSharedPtr<FGameplayTagNode> InTagNode);

	/** Returns true if the user can select tags from the widget */
	UE_API bool CanSelectTags() const;

	/** Determines if the expandable UI that contains the Add New Tag widget should be expanded or collapsed */
	UE_API ECheckBoxState GetAddTagSectionExpansionState() const;

	/** Callback for when the state of the expandable UI section changes */
	UE_API void OnAddTagSectionExpansionStateChanged(ECheckBoxState NewState);

	/** Determines if the expandable UI that contains the Add New Tag Source widget should be expanded or collapsed */
	UE_API ECheckBoxState GetAddSourceSectionExpansionState() const;

	/** Callback for when the state of the expandable tag source UI section changes */
	UE_API void OnAddSourceSectionExpansionStateChanged(ECheckBoxState NewState);

	UE_API void SetContainer(FGameplayTagContainer* OriginalContainer, FGameplayTagContainer* EditedContainer, UObject* OwnerObj);

	/** Opens a dialog window to rename the selected tag */
	UE_API void OpenRenameGameplayTagDialog(TSharedPtr<FGameplayTagNode> GameplayTagNode) const;

	/** Delegate that is fired when a tag is successfully renamed */
	UE_API void OnGameplayTagRenamed(FString OldTagName, FString NewTagName);
	
	/** Populate tag items from the gameplay tags manager. */
	UE_API void GetFilteredGameplayRootTags(const FString& InFilterString, TArray<TSharedPtr<FGameplayTagNode>>& OutNodes) const;
};

#undef UE_API
